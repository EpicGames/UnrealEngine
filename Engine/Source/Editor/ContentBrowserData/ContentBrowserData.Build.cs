// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class ContentBrowserData : ModuleRules
{
	public ContentBrowserData(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"EditorSubsystem",
				"UnrealEd",
				"Projects",
			}
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"CollectionManager",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Slate",
				"SlateCore",
			}
		);
	}
}
