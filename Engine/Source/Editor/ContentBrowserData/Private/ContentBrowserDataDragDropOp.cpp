// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserDataDragDropOp.h"

#include "AssetRegistry/AssetData.h"
#include "AssetThumbnail.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "ContentBrowserDataFilter.h"
#include "ContentBrowserDataSubsystem.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "HAL/PlatformCrt.h"
#include "Templates/UnrealTemplate.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Widgets/Layout/SBox.h"
#include "UObject/NameTypes.h"

class UActorFactory;

TSharedRef<FContentBrowserDataDragDropOp> FContentBrowserDataDragDropOp::New(TArrayView<const FContentBrowserItem> InDraggedItems, FThumbnailOverrideParams InThumbnailOverrideParams)
{
	TSharedRef<FContentBrowserDataDragDropOp> Operation = MakeShared<FContentBrowserDataDragDropOp>();

	Operation->Init(MoveTemp(InDraggedItems));
	Operation->FolderBrushName = InThumbnailOverrideParams.FolderBrushName;
	Operation->FolderShadowBrushName = InThumbnailOverrideParams.FolderShadowBrushName;
	Operation->FolderColorOverride = InThumbnailOverrideParams.FolderColorOverride;

	if (Operation->ShouldOverrideThumbnailWidget())
	{
		const TSharedPtr<SWidget> CustomThumbnailWidget = Operation->GetFolderWidgetDragAndDrop();
		if (CustomThumbnailWidget && CustomThumbnailWidget.IsValid())
		{
			Operation->SetCustomThumbnailWidget(CustomThumbnailWidget.ToSharedRef());
		}
	}

	Operation->Construct();
	return Operation;
}

TSharedRef<FContentBrowserDataDragDropOp> FContentBrowserDataDragDropOp::Legacy_New(TArrayView<const FAssetData> InAssetData, TArrayView<const FString> InAssetPaths, UActorFactory* InActorFactory)
{
	TSharedRef<FContentBrowserDataDragDropOp> Operation = MakeShared<FContentBrowserDataDragDropOp>();

	Operation->LegacyInit(InAssetData, InAssetPaths, InActorFactory);

	Operation->Construct();
	return Operation;
}

void FContentBrowserDataDragDropOp::Init(TArrayView<const FContentBrowserItem> InDraggedItems)
{
	DraggedItems.Append(InDraggedItems.GetData(), InDraggedItems.Num());

	TArray<FAssetData> DraggedAssets;
	TArray<FString> DraggedPackagePaths;

	for (const FContentBrowserItem& DraggedItem : DraggedItems)
	{
		if (DraggedItem.IsFile())
		{
			DraggedFiles.Add(DraggedItem);

			FAssetData ItemAssetData;
			if (DraggedItem.Legacy_TryGetAssetData(ItemAssetData) && !ItemAssetData.IsRedirector())
			{
				DraggedAssets.Add(MoveTemp(ItemAssetData));
			}
		}

		if (DraggedItem.IsFolder())
		{
			DraggedFolders.Add(DraggedItem);

			FName ItemPackagePath;
			if (DraggedItem.Legacy_TryGetPackagePath(ItemPackagePath))
			{
				DraggedPackagePaths.Add(ItemPackagePath.ToString());
			}
		}
	}

	FAssetDragDropOp::Init(MoveTemp(DraggedAssets), MoveTemp(DraggedPackagePaths), nullptr);
}

void FContentBrowserDataDragDropOp::LegacyInit(TArrayView<const FAssetData> InAssetData, TArrayView<const FString> InAssetPaths, UActorFactory* InActorFactory)
{
	UContentBrowserDataSubsystem* ContentBrowserData = GEditor->GetEditorSubsystem<UContentBrowserDataSubsystem>();

	for (const FAssetData& Asset : InAssetData)
	{
		TArray<FName, TInlineAllocator<2>> VirtualAssetPaths;
		ContentBrowserData->Legacy_TryConvertAssetDataToVirtualPaths(Asset, /*bUseFolderPaths*/false, [&VirtualAssetPaths](FName InPath)
		{
			VirtualAssetPaths.Add(InPath);
			return true;
		});

		for (const FName& VirtualAssetPath : VirtualAssetPaths)
		{
			FContentBrowserItem AssetItem = ContentBrowserData->GetItemAtPath(VirtualAssetPath, EContentBrowserItemTypeFilter::IncludeFiles);
			if (AssetItem.IsValid())
			{
				DraggedItems.Add(MoveTemp(AssetItem));
			}
		}
	}

	for (const FString& Path : InAssetPaths)
	{
		TArray<FName, TInlineAllocator<2>> VirtualFolderPaths;
		ContentBrowserData->Legacy_TryConvertPackagePathToVirtualPaths(*Path, [&VirtualFolderPaths](FName InPath)
		{
			VirtualFolderPaths.Add(InPath);
			return true;
		});

		for (const FName& VirtualFolderPath : VirtualFolderPaths)
		{
			FContentBrowserItem FolderItem = ContentBrowserData->GetItemAtPath(VirtualFolderPath, EContentBrowserItemTypeFilter::IncludeFolders);
			if (FolderItem.IsValid())
			{
				DraggedItems.Add(MoveTemp(FolderItem));
			}
		}
	}

	FAssetDragDropOp::Init(TArray<FAssetData>(InAssetData), TArray<FString>(InAssetPaths), InActorFactory);
}

void FContentBrowserDataDragDropOp::InitThumbnail()
{
	if (DraggedFiles.Num() > 0 && ThumbnailSize > 0)
	{
		// Create the thumbnail handle
		AssetThumbnail = MakeShared<FAssetThumbnail>(FAssetData(), ThumbnailSize, ThumbnailSize, UThumbnailManager::Get().GetSharedThumbnailPool());
		if (DraggedFiles[0].UpdateThumbnail(*AssetThumbnail))
		{
			// Request the texture then tick the pool once to render the thumbnail
			AssetThumbnail->GetViewportRenderTargetTexture();
		}
		else
		{
			AssetThumbnail.Reset();
		}
	}
}

bool FContentBrowserDataDragDropOp::HasFiles() const
{
	return DraggedFiles.Num() > 0;
}

bool FContentBrowserDataDragDropOp::HasFolders() const
{
	return DraggedFolders.Num() > 0;
}

int32 FContentBrowserDataDragDropOp::GetTotalCount() const
{
	return DraggedItems.Num();
}

FText FContentBrowserDataDragDropOp::GetFirstItemText() const
{
	if (DraggedFiles.Num() > 0)
	{
		return DraggedFiles[0].GetDisplayName();
	}

	if (DraggedFolders.Num() > 0)
	{
		UContentBrowserDataSubsystem* ContentBrowserData = GEditor->GetEditorSubsystem<UContentBrowserDataSubsystem>();
		return ContentBrowserData->ConvertVirtualPathToDisplay(DraggedFolders[0]);
	}

	return FText::GetEmpty();
}

bool FContentBrowserDataDragDropOp::ShouldOverrideThumbnailWidget() const
{
	return HasFolders() && DraggedFiles.IsEmpty();
}

TSharedPtr<SWidget> FContentBrowserDataDragDropOp::GetFolderWidget() const
{
	const FSlateBrush* FolderBrush = FAppStyle::GetOptionalBrush(FolderBrushName, nullptr, FAppStyle::GetBrush("ContentBrowser.ListViewFolderIcon"));
	const FSlateBrush* FolderShadowBrush = FAppStyle::GetOptionalBrush(FolderShadowBrushName, nullptr , FAppStyle::GetBrush("ContentBrowser.FolderItem.DropShadow"));

	if (FolderBrush && FolderShadowBrush)
	{
		return SNew(SBorder)
			.BorderImage(FolderShadowBrush)
			.Padding(FMargin(0,0,2.0f,2.0f))
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush(FolderBrushName))
				.ColorAndOpacity(FolderColorOverride)
			];
	}
	return nullptr;
}

TSharedPtr<SWidget> FContentBrowserDataDragDropOp::GetFolderWidgetDragAndDrop() const
{
	const TSharedPtr<SWidget> FolderWidget = GetFolderWidget();

	if (!FolderWidget || !FolderWidget.IsValid())
	{
		return nullptr;
	}

	static const FMargin FolderBoxPadding = FMargin(3.f, 4.f, 1.f, 0.f);
	static constexpr float FolderSize = 60.f;

	return SNew(SBorder)
			.Padding(0.f)
			.BorderImage(FAppStyle::GetBrush("ContentBrowser.ThumbnailDragDropBackground"))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.WidthOverride(FolderSize)
				.HeightOverride(FolderSize)
				.Padding(FolderBoxPadding)
				[
					FolderWidget.ToSharedRef()
				]
			];
}
