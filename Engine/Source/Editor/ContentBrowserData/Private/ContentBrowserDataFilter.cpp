// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserDataFilter.h"

#include "ContentBrowserDataSubsystem.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBrowserDataFilter)

FContentBrowserDataFilterList::FContentBrowserDataFilterList(const FContentBrowserDataFilterList& InOther)
{
	SetTo(InOther);
}

FContentBrowserDataFilterList& FContentBrowserDataFilterList::operator=(const FContentBrowserDataFilterList& InOther)
{
	if (this != &InOther)
	{
		SetTo(InOther);
	}
	return *this;
}

TArray<const UScriptStruct*> FContentBrowserDataFilterList::GetFilterTypes() const
{
	TArray<const UScriptStruct*> Types;
	Types.Reserve(TypedFilters.Num());
	for (const FStructOnScope& Filter : TypedFilters)
	{
		Types.Add(CastChecked<const UScriptStruct>(Filter.GetStruct()));
	}
	return Types;
}

void* FContentBrowserDataFilterList::FindOrAddFilter(const UScriptStruct* InFilterType)
{
	check(InFilterType);

	if (void* FilterData = const_cast<void*>(FindFilter(InFilterType)))
	{
		return FilterData;
	}
	
	FStructOnScope& Filter = TypedFilters.Emplace_GetRef(InFilterType);
	return Filter.GetStructMemory();
}

void FContentBrowserDataFilterList::SetFilter(const UScriptStruct* InFilterType, const void* InFilterData)
{
	void* FilterData = FindOrAddFilter(InFilterType);
	check(FilterData && InFilterData);
	InFilterType->CopyScriptStruct(FilterData, InFilterData);
}

const void* FContentBrowserDataFilterList::FindFilter(const UScriptStruct* InFilterType) const
{
	check(InFilterType);

	for (const FStructOnScope& Filter : TypedFilters)
	{
		if (Filter.GetStruct() == InFilterType)
		{
			return Filter.GetStructMemory();
		}
	}

	return nullptr;
}

void FContentBrowserDataFilterList::RemoveFilter(const UScriptStruct* InFilterType)
{
	check(InFilterType);

	for (auto FilterIt = TypedFilters.CreateIterator(); FilterIt; ++FilterIt)
	{
		if (FilterIt->GetStruct() == InFilterType)
		{
			FilterIt.RemoveCurrent();
			break;
		}
	}
}

void FContentBrowserDataFilterList::ClearFilters()
{
	TypedFilters.Reset();
}

void FContentBrowserDataFilterList::SetTo(const FContentBrowserDataFilterList& InOther)
{
	TypedFilters.Reset(InOther.TypedFilters.Num());
	for (const FStructOnScope& OtherFilter : InOther.TypedFilters)
	{
		const UScriptStruct* ScriptStructPtr = CastChecked<UScriptStruct>(OtherFilter.GetStruct());

		FStructOnScope& Filter = TypedFilters.Emplace_GetRef(ScriptStructPtr);
		ScriptStructPtr->CopyScriptStruct(Filter.GetStructMemory(), OtherFilter.GetStructMemory());
	}
}

void FContentBrowserDataFilterCacheIDOwner::Initialaze(UContentBrowserDataSubsystem* InContentBrowserDataSubsystem)
{
	if (DataSource != InContentBrowserDataSubsystem)
	{
		ClearCachedData();

		if (InContentBrowserDataSubsystem)
		{
			UContentBrowserDataSubsystem::FContentBrowserFilterCacheApi::InitializeCacheIDOwner(*InContentBrowserDataSubsystem, *this);
		}
	}
}

void FContentBrowserDataFilterCacheIDOwner::RemoveUnusedCachedData(TArrayView<const FName> InVirtualPathsInUse, const FContentBrowserDataFilter& DataFilter) const
{
	if (const UContentBrowserDataSubsystem* ContentBrowserDataSubsystem = DataSource.Get())
	{
		UContentBrowserDataSubsystem::FContentBrowserFilterCacheApi::RemoveUnusedCachedData(*ContentBrowserDataSubsystem,
			*this,
			InVirtualPathsInUse,
			DataFilter);
	}
}

void FContentBrowserDataFilterCacheIDOwner::ClearCachedData() const
{
	if (const UContentBrowserDataSubsystem* ContentBrowserDataSubsystem = DataSource.Get())
	{
		UContentBrowserDataSubsystem::FContentBrowserFilterCacheApi::ClearCachedData(*ContentBrowserDataSubsystem, *this);
	}
}

void FContentBrowserDataFilterCacheIDOwner::Reset()
{
	ClearCachedData();
	ID = INDEX_NONE;
	DataSource.Reset();
}
