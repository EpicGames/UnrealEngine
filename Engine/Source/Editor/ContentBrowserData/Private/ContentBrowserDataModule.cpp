// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserDataSubsystem.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "IContentBrowserDataModule.h"
#include "Modules/ModuleManager.h"

class FContentBrowserDataModule : public IContentBrowserDataModule
{
public:
	virtual UContentBrowserDataSubsystem* GetSubsystem() const override
	{
		return GEditor ? GEditor->GetEditorSubsystem<UContentBrowserDataSubsystem>() : nullptr;
	}
};

IMPLEMENT_MODULE(FContentBrowserDataModule, ContentBrowserData);
