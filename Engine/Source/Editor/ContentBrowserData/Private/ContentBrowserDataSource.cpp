// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserDataSource.h"

#include "Containers/StringView.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserItemData.h"
#include "ContentBrowserItemPath.h"
#include "ContentBrowserVirtualFolderDataPayload.h"
#include "CoreTypes.h"
#include "Features/IModularFeatures.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/PackageName.h"
#include "Misc/StringBuilder.h"
#include "Settings/ContentBrowserSettings.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/UnrealNames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBrowserDataSource)

#define LOCTEXT_NAMESPACE "ContentBrowserDataSource"

FName UContentBrowserDataSource::GetModularFeatureTypeName()
{
	static const FName ModularFeatureTypeName = "ContentBrowserDataSource";
	return ModularFeatureTypeName;
}

void UContentBrowserDataSource::RegisterDataSource()
{
	IModularFeatures::Get().RegisterModularFeature(GetModularFeatureTypeName(), this);
}

void UContentBrowserDataSource::UnregisterDataSource()
{
	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureTypeName(), this);
}

void UContentBrowserDataSource::Initialize(const bool InAutoRegister)
{
	bIsInitialized = true;

	if (InAutoRegister)
	{
		RegisterDataSource();
	}
}

void UContentBrowserDataSource::Shutdown()
{
	UnregisterDataSource();

	bIsInitialized = false;
}

void UContentBrowserDataSource::BeginDestroy()
{
	Shutdown();

	Super::BeginDestroy();
}

void UContentBrowserDataSource::SetDataSink(IContentBrowserItemDataSink* InDataSink)
{
	if (InDataSink && (InDataSink != DataSink))
	{
		SetVirtualPathTreeNeedsRebuild();
	}

	DataSink = InDataSink;

	RefreshVirtualPathTreeIfNeeded();
}

bool UContentBrowserDataSource::IsInitialized() const
{
	return bIsInitialized;
}

void UContentBrowserDataSource::Tick(const float InDeltaTime)
{
}

void UContentBrowserDataSource::BuildRootPathVirtualTree()
{
	RootPathVirtualTree.Reset();
	VirtualPathTreeRulesCachedState.bShowAllFolder = GetDefault<UContentBrowserSettings>()->bShowAllFolder;
	VirtualPathTreeRulesCachedState.bOrganizeFolders = GetDefault<UContentBrowserSettings>()->bOrganizeFolders;
	bVirtualPathTreeNeedsRebuild = false;
}

void UContentBrowserDataSource::SetVirtualPathTreeNeedsRebuild()
{
	bVirtualPathTreeNeedsRebuild = true;
}

void UContentBrowserDataSource::RefreshVirtualPathTreeIfNeeded()
{
	if (bVirtualPathTreeNeedsRebuild ||
		GetDefault<UContentBrowserSettings>()->bShowAllFolder != VirtualPathTreeRulesCachedState.bShowAllFolder ||
		GetDefault<UContentBrowserSettings>()->bOrganizeFolders != VirtualPathTreeRulesCachedState.bOrganizeFolders)
	{
		BuildRootPathVirtualTree();
	}
}

bool UContentBrowserDataSource::IsVirtualPathUnderMountRoot(const FName InPath) const
{
	FName ConvertedPath;
	return TryConvertVirtualPath(InPath, ConvertedPath) != EContentBrowserPathType::None;
}

EContentBrowserPathType UContentBrowserDataSource::TryConvertVirtualPath(const FStringView InPath, FStringBuilderBase& OutPath) const
{
	return RootPathVirtualTree.TryConvertVirtualPathToInternal(InPath, OutPath);
}

EContentBrowserPathType UContentBrowserDataSource::TryConvertVirtualPath(const FStringView InPath, FString& OutPath) const
{
	return RootPathVirtualTree.TryConvertVirtualPathToInternal(InPath, OutPath);
}

EContentBrowserPathType UContentBrowserDataSource::TryConvertVirtualPath(const FName InPath, FName& OutPath) const
{
	return RootPathVirtualTree.TryConvertVirtualPathToInternal(InPath, OutPath);
}

bool UContentBrowserDataSource::TryConvertVirtualPathToInternal(const FName InPath, FName& OutInternalPath)
{
	if (TryConvertVirtualPath(InPath, OutInternalPath) == EContentBrowserPathType::Internal)
	{
		return true;
	}

	OutInternalPath = NAME_None;
	return false;
}

bool UContentBrowserDataSource::TryConvertInternalPathToVirtual(const FName InInternalPath, FName& OutPath)
{
	static const FName RootPath = "/";

	// Special case "/" cannot be converted or remapped
	if (InInternalPath.IsNone() || InInternalPath == RootPath || !DataSink)
	{
		OutPath = InInternalPath;
		return true;
	}

	FNameBuilder OutPathBuffer;
	DataSink->ConvertInternalPathToVirtual(FNameBuilder(InInternalPath), OutPathBuffer);
	OutPath = FName(OutPathBuffer);

	return true;
}

void UContentBrowserDataSource::RemoveUnusedCachedFilterData(const FContentBrowserDataFilterCacheIDOwner& IDOwner, TArrayView<const FName> InVirtualPathsInUse, const FContentBrowserDataFilter& DataFilter)
{
}

void UContentBrowserDataSource::ClearCachedFilterData(const FContentBrowserDataFilterCacheIDOwner& IDOwner)
{
}

void UContentBrowserDataSource::RootPathAdded(const FStringView InInternalPath)
{
	// Trim trailing slash
	FStringView Path(InInternalPath);
	if (Path.Len() > 1 && Path[Path.Len() - 1] == TEXT('/'))
	{
		Path.LeftChopInline(1);
	}

	// Handle root containing multiple folders such as /Game/FirstPerson
	int32 SecondSlashIndex = INDEX_NONE;
	if (Path.Len() > 1 && Path.RightChop(1).FindChar(TEXT('/'), SecondSlashIndex))
	{
		const FStringView FirstPath = Path.Left(SecondSlashIndex + 1);

		FName FirstParentVirtualPath;
		TryConvertInternalPathToVirtual(FName(FirstPath), FirstParentVirtualPath);

		bool bIsFullyVirtual = false;
		if (RootPathVirtualTree.PathExists(FirstParentVirtualPath, bIsFullyVirtual))
		{
			if (bIsFullyVirtual == false)
			{
				return;
			}
		}
	}

	FName VirtualPath;
	FName PathFName(Path);
	TryConvertInternalPathToVirtual(PathFName, VirtualPath);
	RootPathVirtualTree.CachePath(VirtualPath, PathFName, [](FName AddedPath){});
}

void UContentBrowserDataSource::RootPathRemoved(const FStringView InInternalPath)
{
	// Trim trailing slash
	FStringView Path(InInternalPath);
	if (Path.Len() > 1 && Path[Path.Len() - 1] == TEXT('/'))
	{
		Path.LeftChopInline(1);
	}

	FName VirtualPath;
	TryConvertInternalPathToVirtual(FName(Path), VirtualPath);
	RootPathVirtualTree.RemovePath(VirtualPath, [](FName RemovedPath){});
}

void UContentBrowserDataSource::CompileFilter(const FName InPath, const FContentBrowserDataFilter& InFilter, FContentBrowserDataCompiledFilter& OutCompiledFilter)
{
}

void UContentBrowserDataSource::EnumerateItemsMatchingFilter(const FContentBrowserDataCompiledFilter& InFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback)
{
}

void UContentBrowserDataSource::EnumerateItemsMatchingFilter(const FContentBrowserDataCompiledFilter& InFilter, const TGetOrEnumerateSink<FContentBrowserItemData>& InSink)
{
	// Call older API function if this wasn't overridden
	EnumerateItemsMatchingFilter(InFilter, [&InSink](FContentBrowserItemData&& Item) { return InSink.ProduceItem(MoveTemp(Item)); });
}

void UContentBrowserDataSource::EnumerateItemsAtPath(const FName InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback)
{
}

void UContentBrowserDataSource::EnumerateItemsAtPath(const FName InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, const TGetOrEnumerateSink<FContentBrowserItemData>& InSink)
{
	EnumerateItemsAtPath(InPath, InItemTypeFilter, [&InSink](FContentBrowserItemData&& Item) { return InSink.ProduceItem(MoveTemp(Item)); });
}

bool UContentBrowserDataSource::EnumerateItemsAtPaths(const TArrayView<FContentBrowserItemPath> InPaths, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback)
{
	return true;
}

void UContentBrowserDataSource::EnumerateItemsAtUserProvidedPath(const FStringView InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback)
{
}

void UContentBrowserDataSource::EnumerateItemsAtUserProvidedPath(const FStringView InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, const TGetOrEnumerateSink<FContentBrowserItemData>& InSink)
{
	EnumerateItemsAtUserProvidedPath(InPath, InItemTypeFilter, [&InSink](FContentBrowserItemData&& Item) { return InSink.ProduceItem(MoveTemp(Item)); });
}

bool UContentBrowserDataSource::EnumerateItemsForObjects(const TArrayView<UObject*> InObjects, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback)
{
	return true;
}

TArray<FContentBrowserItemPath> UContentBrowserDataSource::GetAliasesForPath(const FSoftObjectPath& InInternalPath) const
{
	return TArray<FContentBrowserItemPath>();
}

bool UContentBrowserDataSource::GetItemAliasedPath(const FContentBrowserItemData& InItemData, FSoftObjectPath& OutObjectPath) const
{
	return false;
}

TArray<FContentBrowserItemPath> UContentBrowserDataSource::GetAliasesForPath(FName InInternalPath) const
{
	return GetAliasesForPath(FSoftObjectPath(InInternalPath.ToString()));
}

bool UContentBrowserDataSource::IsDiscoveringItems(FText* OutStatus)
{
	return false;
}

bool UContentBrowserDataSource::PrioritizeSearchPath(const FName InPath)
{
	return false;
}

bool UContentBrowserDataSource::IsFolderVisible(const FName Path, const EContentBrowserIsFolderVisibleFlags Flags, const FContentBrowserFolderContentsFilter& ContentsFilter)
{
	return true;
}

bool UContentBrowserDataSource::CanCreateFolder(const FName InPath, FText* OutErrorMsg)
{
	return false;
}

bool UContentBrowserDataSource::CreateFolder(const FName InPath, const TSharedPtr<IContentBrowserHideFolderIfEmptyFilter>& HideFolderIfEmptyFilter, FContentBrowserItemDataTemporaryContext& OutPendingItem)
{
	return false;
}

bool UContentBrowserDataSource::DoesItemPassFilter(const FContentBrowserItemData& InItem, const FContentBrowserDataCompiledFilter& InFilter)
{
	return false;
}

bool UContentBrowserDataSource::ConvertItemForFilter(FContentBrowserItemData& Item, const FContentBrowserDataCompiledFilter& InFilter)
{
	return false;
}

bool UContentBrowserDataSource::GetItemAttribute(const FContentBrowserItemData& InItem, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue)
{
	return false;
}

bool UContentBrowserDataSource::GetItemAttributes(const FContentBrowserItemData& InItem, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues)
{
	return false;
}

bool UContentBrowserDataSource::GetItemPhysicalPath(const FContentBrowserItemData& InItem, FString& OutDiskPath)
{
	return false;
}

bool UContentBrowserDataSource::GetItemAssetAccessSpecifier(const FContentBrowserItemData& InItem, EAssetAccessSpecifier& OutAssetAccessSpecifier)
{
	return false;
}

bool UContentBrowserDataSource::CanModifyItemAssetAccessSpecifier(const FContentBrowserItemData& InItem)
{
	return false;
}

bool UContentBrowserDataSource::IsItemDirty(const FContentBrowserItemData& InItem)
{
	return false;
}

bool UContentBrowserDataSource::CanEditItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	return false;
}

bool UContentBrowserDataSource::EditItem(const FContentBrowserItemData& InItem)
{
	return false;
}

bool UContentBrowserDataSource::BulkEditItems(TArrayView<const FContentBrowserItemData> InItems)
{
	bool bSuccess = false;
	for (const FContentBrowserItemData& Item : InItems)
	{
		bSuccess |= EditItem(Item);
	}
	return bSuccess;
}

bool UContentBrowserDataSource::CanViewItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	return false;
}

bool UContentBrowserDataSource::ViewItem(const FContentBrowserItemData& InItem)
{
	return false;
}

bool UContentBrowserDataSource::BulkViewItems(TArrayView<const FContentBrowserItemData> InItems)
{
	bool bSuccess = false;
	for (const FContentBrowserItemData& Item : InItems)
	{
		bSuccess |= ViewItem(Item);
	}
	return bSuccess;
}

bool UContentBrowserDataSource::CanPreviewItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	return false;
}

bool UContentBrowserDataSource::PreviewItem(const FContentBrowserItemData& InItem)
{
	return false;
}

bool UContentBrowserDataSource::BulkPreviewItems(TArrayView<const FContentBrowserItemData> InItems)
{
	bool bSuccess = false;
	for (const FContentBrowserItemData& Item : InItems)
	{
		bSuccess |= PreviewItem(Item);
	}
	return bSuccess;
}

bool UContentBrowserDataSource::CanDuplicateItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	return false;
}

bool UContentBrowserDataSource::DuplicateItem(const FContentBrowserItemData& InItem, FContentBrowserItemDataTemporaryContext& OutPendingItem)
{
	return false;
}

bool UContentBrowserDataSource::BulkDuplicateItems(TArrayView<const FContentBrowserItemData> InItems, TArray<FContentBrowserItemData>& OutNewItems)
{
	return false;
}

bool UContentBrowserDataSource::CanSaveItem(const FContentBrowserItemData& InItem, const EContentBrowserItemSaveFlags InSaveFlags, FText* OutErrorMsg)
{
	return false;
}

bool UContentBrowserDataSource::SaveItem(const FContentBrowserItemData& InItem, const EContentBrowserItemSaveFlags InSaveFlags)
{
	return false;
}

bool UContentBrowserDataSource::BulkSaveItems(TArrayView<const FContentBrowserItemData> InItems, const EContentBrowserItemSaveFlags InSaveFlags)
{
	bool bSuccess = false;
	for (const FContentBrowserItemData& Item : InItems)
	{
		bSuccess |= SaveItem(Item, InSaveFlags);
	}
	return bSuccess;
}

bool UContentBrowserDataSource::CanDeleteItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	return false;
}

bool UContentBrowserDataSource::DeleteItem(const FContentBrowserItemData& InItem)
{
	return false;
}

bool UContentBrowserDataSource::BulkDeleteItems(TArrayView<const FContentBrowserItemData> InItems)
{
	bool bSuccess = false;
	for (const FContentBrowserItemData& Item : InItems)
	{
		bSuccess |= DeleteItem(Item);
	}
	return bSuccess;
}

bool UContentBrowserDataSource::CanPrivatizeItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg)
{
	return false;
}

bool UContentBrowserDataSource::PrivatizeItem(const FContentBrowserItemData& InItem, const EAssetAccessSpecifier InAssetAccessSpecifier)
{
	return false;
}

bool UContentBrowserDataSource::BulkPrivatizeItems(TArrayView<const FContentBrowserItemData> InItems, const EAssetAccessSpecifier InAssetAccessSpecifier)
{
	bool bSuccess = false;
	for (const FContentBrowserItemData& Item : InItems)
	{
		bSuccess |= PrivatizeItem(Item, InAssetAccessSpecifier);
	}
	return bSuccess;
}

bool UContentBrowserDataSource::CanRenameItem(const FContentBrowserItemData& InItem, const FString* InNewName, const IContentBrowserHideFolderIfEmptyFilter* HideFolderIfEmptyFilter, FText* OutErrorMsg)
{
	return false;
}

bool UContentBrowserDataSource::RenameItem(const FContentBrowserItemData& InItem, const FString& InNewName, FContentBrowserItemData& OutNewItem)
{
	return false;
}

bool UContentBrowserDataSource::CanCopyItem(const FContentBrowserItemData& InItem, const FName InDestPath, FText* OutErrorMsg)
{
	return false;
}

bool UContentBrowserDataSource::CopyItem(const FContentBrowserItemData& InItem, const FName InDestPath)
{
	return false;
}

bool UContentBrowserDataSource::BulkCopyItems(TArrayView<const FContentBrowserItemData> InItems, const FName InDestPath)
{
	bool bSuccess = false;
	for (const FContentBrowserItemData& Item : InItems)
	{
		bSuccess |= CopyItem(Item, InDestPath);
	}
	return bSuccess;
}

bool UContentBrowserDataSource::CanMoveItem(const FContentBrowserItemData& InItem, const FName InDestPath, FText* OutErrorMsg)
{
	return false;
}

bool UContentBrowserDataSource::MoveItem(const FContentBrowserItemData& InItem, const FName InDestPath)
{
	return false;
}

bool UContentBrowserDataSource::BulkMoveItems(TArrayView<const FContentBrowserItemData> InItems, const FName InDestPath)
{
	bool bSuccess = false;
	for (const FContentBrowserItemData& Item : InItems)
	{
		bSuccess |= MoveItem(Item, InDestPath);
	}
	return bSuccess;
}

bool UContentBrowserDataSource::AppendItemReference(const FContentBrowserItemData& InItem, FString& InOutStr)
{
	return false;
}

bool UContentBrowserDataSource::AppendItemObjectPath(const FContentBrowserItemData& InItem, FString& InOutStr)
{
	return false;
}

bool UContentBrowserDataSource::AppendItemPackageName(const FContentBrowserItemData& InItem, FString& InOutStr)
{
	return false;
}

bool UContentBrowserDataSource::UpdateThumbnail(const FContentBrowserItemData& InItem, FAssetThumbnail& InThumbnail)
{
	return false;
}

TSharedPtr<FDragDropOperation> UContentBrowserDataSource::CreateCustomDragOperation(TArrayView<const FContentBrowserItemData> InItems)
{
	return nullptr;
}

bool UContentBrowserDataSource::HandleDragEnterItem(const FContentBrowserItemData& InItem, const FDragDropEvent& InDragDropEvent)
{
	return false;
}

bool UContentBrowserDataSource::HandleDragOverItem(const FContentBrowserItemData& InItem, const FDragDropEvent& InDragDropEvent)
{
	return false;
}

bool UContentBrowserDataSource::HandleDragLeaveItem(const FContentBrowserItemData& InItem, const FDragDropEvent& InDragDropEvent)
{
	return false;
}

bool UContentBrowserDataSource::HandleDragDropOnItem(const FContentBrowserItemData& InItem, const FDragDropEvent& InDragDropEvent)
{
	return false;
}

bool UContentBrowserDataSource::TryGetCollectionId(const FContentBrowserItemData& InItem, FSoftObjectPath& OutCollectionId)
{
	return false;
}

bool UContentBrowserDataSource::Legacy_TryGetPackagePath(const FContentBrowserItemData& InItem, FName& OutPackagePath)
{
	return false;
}

bool UContentBrowserDataSource::Legacy_TryGetAssetData(const FContentBrowserItemData& InItem, FAssetData& OutAssetData)
{
	return false;
}

bool UContentBrowserDataSource::Legacy_TryConvertPackagePathToVirtualPath(const FName InPackagePath, FName& OutPath)
{
	return false;
}

bool UContentBrowserDataSource::Legacy_TryConvertAssetDataToVirtualPath(const FAssetData& InAssetData, const bool InUseFolderPaths, FName& OutPath)
{
	return false;
}

void UContentBrowserDataSource::QueueItemDataUpdate(FContentBrowserItemDataUpdate&& InUpdate)
{
	if (DataSink)
	{
		DataSink->QueueItemDataUpdate(MoveTemp(InUpdate));
	}
}

void UContentBrowserDataSource::NotifyItemDataRefreshed()
{
	if (DataSink)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		DataSink->NotifyItemDataRefreshed();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

FContentBrowserItemData UContentBrowserDataSource::CreateVirtualFolderItem(const FName InFolderPath)
{
	const FString FolderItemName = FPackageName::GetShortName(InFolderPath);

	static const FName AllRootPath = "/All";

	FText FolderDisplayNameOverride;
	if (InFolderPath == AllRootPath)
	{
		FolderDisplayNameOverride = LOCTEXT("AllFolderDisplayName", "All");
	}
	else if (FolderItemName == TEXT("GameData"))
	{
		FolderDisplayNameOverride = LOCTEXT("GameDataFolderDisplayName", "Game Data");
	}
	else if (FolderItemName == TEXT("EngineData"))
	{
		FolderDisplayNameOverride = LOCTEXT("EngineDataFolderDisplayName", "Engine");
	}

	return FContentBrowserItemData(this,
		EContentBrowserItemFlags::Type_Folder,
		InFolderPath,
		*FolderItemName,
		MoveTemp(FolderDisplayNameOverride),
		nullptr,
		FName() // Virtual folders have no internal path 
		);
}

#undef LOCTEXT_NAMESPACE
