// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserDataSubsystem.h"

#include "Containers/StringView.h"
#include "Containers/Ticker.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserItemPath.h"
#include "Editor.h"
#include "Features/IModularFeatures.h"
#include "Framework/Application/SlateApplication.h"
#include "GetOrEnumerateSink.h"
#include "HAL/IConsoleManager.h"
#include "IContentBrowserDataModule.h"
#include "Interfaces/IPluginManager.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/PackageName.h"
#include "Misc/PathViews.h"
#include "Misc/PathViews.h"
#include "Misc/StringBuilder.h"
#include "PluginDescriptor.h"
#include "Settings/ContentBrowserSettings.h"
#include "Stats/Stats.h"
#include "String/RemoveFrom.h"
#include "Templates/Function.h"
#include "Templates/Less.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/GarbageCollection.h"
#include "UObject/UObjectThreadContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBrowserDataSubsystem)

class FSubsystemCollectionBase;
class UObject;
struct FAssetData;

DEFINE_LOG_CATEGORY_STATIC(LogContentBrowserDataSubsystem, Log, All);

namespace ContentBrowserDataSubsystem
{
	FAutoConsoleCommand CVarContentBrowserDebug_TryConvertVirtualPath = FAutoConsoleCommand(
		TEXT("ContentBrowser.Debug.TryConvertVirtualPath"),
		TEXT("Try to convert virtual path"),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			if (Args.Num() > 0)
			{
				const FString& VirtualPath = Args[0];
				FNameBuilder ConvertedPath;
				EContentBrowserPathType PathType = IContentBrowserDataModule::Get().GetSubsystem()->TryConvertVirtualPath(VirtualPath, ConvertedPath);
				UE_LOGF(LogContentBrowserDataSubsystem, Log, "InputVirtualPath: %ls, ConvertedPath: %ls, ConvertedPathType: %ls", *VirtualPath, *ConvertedPath, *UEnum::GetValueAsString(PathType));
			}
		}
	));

	FAutoConsoleCommand CVarContentBrowserDebug_ConvertInternalPathToVirtual = FAutoConsoleCommand(
		TEXT("ContentBrowser.Debug.ConvertInternalPathToVirtual"),
		TEXT("Convert internal path"),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			if (Args.Num() > 0)
			{
				const FString& InternalPath = Args[0];
				FNameBuilder ConvertedPath;
				IContentBrowserDataModule::Get().GetSubsystem()->ConvertInternalPathToVirtual(InternalPath, ConvertedPath);
				UE_LOGF(LogContentBrowserDataSubsystem, Log, "InputInternalPath: %ls, ConvertedVirtualPath: %ls", *InternalPath, *ConvertedPath);
			}
		}
	));

	class FDefaultHideFolderIfEmptyFilter : public IContentBrowserHideFolderIfEmptyFilter
	{
	public:
		FDefaultHideFolderIfEmptyFilter()
			: GameDevelopersPath(FPackageName::FilenameToLongPackageName(FPaths::GameDevelopersDir()))
		{
		}

		bool HideFolderIfEmpty(FName Path, FStringView PathString) const override
		{
			static const FName PublicCollectionsPath("/Game/Collections");

			// Hide public collection folder.
			if (Path == PublicCollectionsPath)
			{
				return true;
			}

			// Hide private collection folders.
			if (PathString.StartsWith(GameDevelopersPath))
			{
				for (int32 Index = GameDevelopersPath.Len(); Index < PathString.Len(); ++Index)
				{
					// Scan past the developer name in /Game/Developers/<developer>/Collections.
					if (PathString[Index] == TEXT('/'))
					{
						return PathString.RightChop(Index + 1).Equals(TEXT("Collections"));
					}
				}
			}

			return false;
		}

	private:
		const FString GameDevelopersPath;
	};

	class FMergedHideFolderIfEmptyFilter : public IContentBrowserHideFolderIfEmptyFilter
	{
	public:
		FMergedHideFolderIfEmptyFilter(TArray<TSharedPtr<IContentBrowserHideFolderIfEmptyFilter>>&& InHideFolderIfEmptyFilters)
			: HideFolderIfEmptyFilters(MoveTemp(InHideFolderIfEmptyFilters))
		{
		}

		bool HideFolderIfEmpty(FName Path, FStringView PathString) const override
		{
			for (const TSharedPtr<IContentBrowserHideFolderIfEmptyFilter>& HideFolderIfEmptyFilter : HideFolderIfEmptyFilters)
			{
				if (HideFolderIfEmptyFilter->HideFolderIfEmpty(Path, PathString))
				{
					return true;
				}
			}

			return false;
		}

	private:
		const TArray<TSharedPtr<IContentBrowserHideFolderIfEmptyFilter>> HideFolderIfEmptyFilters;
	};
}

UContentBrowserDataSubsystem::UContentBrowserDataSubsystem()
	: EditableFolderPermissionList(MakeShared<FPathPermissionList>())
{

}
void UContentBrowserDataSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	AllFolderPrefix = TEXT("/All");

	DefaultPathViewSpecialSortFolders.Reset();
	DefaultPathViewSpecialSortFolders.Add(TEXT("/Game"));
	DefaultPathViewSpecialSortFolders.Add(TEXT("/Plugins"));
	DefaultPathViewSpecialSortFolders.Add(TEXT("/Engine"));
	DefaultPathViewSpecialSortFolders.Add(TEXT("/EngineData"));
	SetPathViewSpecialSortFolders(GetDefaultPathViewSpecialSortFolders());

	IModularFeatures& ModularFeatures = IModularFeatures::Get();

	{
		const FName DataSourceFeatureName = UContentBrowserDataSource::GetModularFeatureTypeName();

		const int32 AvailableDataSourcesCount = ModularFeatures.GetModularFeatureImplementationCount(DataSourceFeatureName);
		for (int32 AvailableDataSourcesIndex = 0; AvailableDataSourcesIndex < AvailableDataSourcesCount; ++AvailableDataSourcesIndex)
		{
			HandleDataSourceRegistered(DataSourceFeatureName, ModularFeatures.GetModularFeatureImplementation(DataSourceFeatureName, AvailableDataSourcesIndex));
		}

		/**
		 * If any view already exist refresh them now instead of waiting.
		 * This avoid asking for the view that where just created to refresh their data next frame during the editor initialization.
		 */ 
		bPendingItemDataRefreshedNotification = false;
		ItemDataRefreshedDelegate.Broadcast();
	}

	ModularFeatures.OnModularFeatureRegistered().AddUObject(this, &UContentBrowserDataSubsystem::HandleDataSourceRegistered);
	ModularFeatures.OnModularFeatureUnregistered().AddUObject(this, &UContentBrowserDataSubsystem::HandleDataSourceUnregistered);

	DefaultHideFolderIfEmptyFilter = MakeShared<ContentBrowserDataSubsystem::FDefaultHideFolderIfEmptyFilter>();

	FEditorDelegates::BeginPIE.AddUObject(this, &UContentBrowserDataSubsystem::OnBeginPIE);
	FEditorDelegates::EndPIE.AddUObject(this, &UContentBrowserDataSubsystem::OnEndPIE);

	FPackageName::OnContentPathMounted().AddUObject(this, &UContentBrowserDataSubsystem::OnContentPathMounted);

	// Tick during normal operation
	TickHandle = FTSTicker::GetCoreTicker().AddTicker(TEXT("ContentBrowserData"), 0.1f, [this](const float InDeltaTime)
	{
		Tick(InDeltaTime);
		return true;
	});

	// Tick during modal dialog operation
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetOnModalLoopTickEvent().AddUObject(this, &UContentBrowserDataSubsystem::Tick);
	}
}

void UContentBrowserDataSubsystem::Deinitialize()
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	ModularFeatures.OnModularFeatureRegistered().RemoveAll(this);
	ModularFeatures.OnModularFeatureUnregistered().RemoveAll(this);

	FEditorDelegates::BeginPIE.RemoveAll(this);
	FEditorDelegates::EndPIE.RemoveAll(this);

	FPackageName::OnContentPathMounted().RemoveAll(this);

	DeactivateAllDataSources();
	ActiveDataSources.Reset();
	AvailableDataSources.Reset();
	ActiveDataSourcesDiscoveringContent.Reset();
	DefaultHideFolderIfEmptyFilter.Reset();

	if (TickHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickHandle);
		TickHandle.Reset();
	}

	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetOnModalLoopTickEvent().RemoveAll(this);
	}
}

bool UContentBrowserDataSubsystem::ActivateDataSource(const FName Name)
{
	EnabledDataSources.AddUnique(Name);

	if (!ActiveDataSources.Contains(Name))
	{
		if (UContentBrowserDataSource* DataSource = AvailableDataSources.FindRef(Name))
		{
			DataSource->SetDataSink(this);
			ActiveDataSources.Add(Name, DataSource);
			ActiveDataSourcesDiscoveringContent.Add(Name);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			NotifyItemDataRefreshed();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			return true;
		}
		else
		{
			// TODO: Log warning
		}
	}

	return false;
}

bool UContentBrowserDataSubsystem::DeactivateDataSource(const FName Name)
{
	EnabledDataSources.Remove(Name);

	if (UContentBrowserDataSource* DataSource = ActiveDataSources.FindRef(Name))
	{
		DataSource->SetDataSink(nullptr);
		ActiveDataSources.Remove(Name);
		ActiveDataSourcesDiscoveringContent.Remove(Name);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		NotifyItemDataRefreshed();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return true;
	}

	return false;
}

void UContentBrowserDataSubsystem::ActivateAllDataSources()
{
	if (ActiveDataSources.Num() == AvailableDataSources.Num())
	{
		// Everything is already active - nothing to do
		return;
	}

	ActiveDataSources = AvailableDataSources;
	for (const auto& ActiveDataSourcePair : ActiveDataSources)
	{
		ActiveDataSourcePair.Value->SetDataSink(this);
		ActiveDataSourcesDiscoveringContent.Add(ActiveDataSourcePair.Key);

		// Merge this array as it may contain sources that we've not yet discovered, so can't activate yet
		EnabledDataSources.AddUnique(ActiveDataSourcePair.Key);
	}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	NotifyItemDataRefreshed();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UContentBrowserDataSubsystem::DeactivateAllDataSources()
{
	if (ActiveDataSources.Num() == 0)
	{
		// Everything is already deactivated - nothing to do
		return;
	}

	for (const auto& ActiveDataSourcePair : ActiveDataSources)
	{
		ActiveDataSourcePair.Value->SetDataSink(nullptr);
	}
	ActiveDataSources.Reset();
	EnabledDataSources.Reset();
	ActiveDataSourcesDiscoveringContent.Reset();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	NotifyItemDataRefreshed();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

TArray<FName> UContentBrowserDataSubsystem::GetAvailableDataSources() const
{
	TArray<FName> AvailableDataSourceNames;
	AvailableDataSources.GenerateKeyArray(AvailableDataSourceNames);
	return AvailableDataSourceNames;
}

TArray<FName> UContentBrowserDataSubsystem::GetActiveDataSources() const
{
	TArray<FName> ActiveDataSourceNames;
	ActiveDataSources.GenerateKeyArray(ActiveDataSourceNames);
	return ActiveDataSourceNames;
}

FOnContentBrowserItemDataUpdated& UContentBrowserDataSubsystem::OnItemDataUpdated()
{
	return ItemDataUpdatedDelegate;
}

FOnContentBrowserItemDataRefreshed& UContentBrowserDataSubsystem::OnItemDataRefreshed()
{
	return ItemDataRefreshedDelegate;
}

FOnContentBrowserItemDataDiscoveryComplete& UContentBrowserDataSubsystem::OnItemDataDiscoveryComplete()
{
	return ItemDataDiscoveryCompleteDelegate;
}

void UContentBrowserDataSubsystem::CompileFilter(const FName InPath, const FContentBrowserDataFilter& InFilter, FContentBrowserDataCompiledFilter& OutCompiledFilter) const
{
	OutCompiledFilter.ItemTypeFilter = InFilter.ItemTypeFilter;
	OutCompiledFilter.ItemCategoryFilter = InFilter.ItemCategoryFilter;
	OutCompiledFilter.ItemAttributeFilter = InFilter.ItemAttributeFilter;

	for (const auto& ActiveDataSourcePair : ActiveDataSources)
	{
		UContentBrowserDataSource* DataSource = ActiveDataSourcePair.Value;
		FName ConvertedPath;
		const EContentBrowserPathType ConvertedPathType = DataSource->TryConvertVirtualPath(InPath, ConvertedPath);
		if (ConvertedPathType != EContentBrowserPathType::None)
		{
			// The requested path is managed by this data source, so compile the filter for it
			DataSource->CompileFilter(InPath, InFilter, OutCompiledFilter);
		}
	}
}

void UContentBrowserDataSubsystem::EnumerateItemsMatchingFilter(const FContentBrowserDataCompiledFilter& InFilter, TFunctionRef<bool(FContentBrowserItem&&)> InCallback) const
{
	EnumerateItemsMatchingFilter(InFilter, [&InCallback](FContentBrowserItemData&& InItemData)
	{
		checkf(InItemData.IsValid(), TEXT("Enumerated items must be valid!"));
		return InCallback(FContentBrowserItem(MoveTemp(InItemData)));
	});
}

void UContentBrowserDataSubsystem::EnumerateItemsMatchingFilter(const FContentBrowserDataCompiledFilter& InFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) const
{
	EnumerateItemsMatchingFilter(InFilter, TGetOrEnumerateSink<FContentBrowserItemData>(InCallback));
}

void UContentBrowserDataSubsystem::EnumerateItemsMatchingFilter(const FContentBrowserDataCompiledFilter& InFilter, const TGetOrEnumerateSink<FContentBrowserItemData>& InSink) const
{
	for (const TPair<FName, UContentBrowserDataSource*>& ActiveDataSourcePair : ActiveDataSources)
	{
		UContentBrowserDataSource* DataSource = ActiveDataSourcePair.Value;

		if (const FContentBrowserDataFilterList* FilterList = InFilter.CompiledFilters.Find(DataSource))
		{
			// Does data source have dummy paths down to its mount root that we also have to emit callbacks for?
			if (const FContentBrowserCompiledSubsystemFilter* SubsystemFilter = FilterList->FindFilter<FContentBrowserCompiledSubsystemFilter>())
			{
				for (const FName& MountRootPart : SubsystemFilter->MountRootsToEnumerate)
				{
					check(EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders));

					const FString MountLeafName = FPackageName::GetShortName(MountRootPart);
					FName InternalPath; // Virtual folders have no internal path 
					InSink.ProduceItem(FContentBrowserItemData(DataSource, EContentBrowserItemFlags::Type_Folder, MountRootPart, *MountLeafName, FText(), nullptr, InternalPath));
				}
			}

			// Fully virtual folders are ones used purely for display purposes such as /All or /All/Plugins
			if (const FContentBrowserCompiledVirtualFolderFilter* VirtualFolderFilter = FilterList->FindFilter<FContentBrowserCompiledVirtualFolderFilter>())
			{
				if (EnumHasAnyFlags(InFilter.ItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders))
				{
					for (const auto& It : VirtualFolderFilter->CachedSubPaths)
					{
						// how do we skip over this item if not included (Engine Content, Engine Plugins, C++ Classes, etc..)
						InSink.ProduceItem(FContentBrowserItemData(It.Value));
					}
				}
			}
		}

		DataSource->EnumerateItemsMatchingFilter(InFilter, InSink);
	}
}

void UContentBrowserDataSubsystem::EnumerateItemsUnderPath(const FName InPath, const FContentBrowserDataFilter& InFilter, TFunctionRef<bool(FContentBrowserItem&&)> InCallback) const
{
	EnumerateItemsUnderPath(InPath, InFilter, [&InCallback](FContentBrowserItemData&& InItemData)
	{
		checkf(InItemData.IsValid(), TEXT("Enumerated items must be valid!"));
		return InCallback(FContentBrowserItem(MoveTemp(InItemData)));
	});
}

void UContentBrowserDataSubsystem::EnumerateItemsUnderPath(const FName InPath, const FContentBrowserDataFilter& InFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) const
{
	FContentBrowserDataCompiledFilter CompiledFilter;
	CompileFilter(InPath, InFilter, CompiledFilter);
	EnumerateItemsMatchingFilter(CompiledFilter, MoveTemp(InCallback));
}

void UContentBrowserDataSubsystem::EnumerateItemsUnderPath(const FName InPath, const FContentBrowserDataFilter& InFilter, const TGetOrEnumerateSink<FContentBrowserItemData>& InSink) const
{
	FContentBrowserDataCompiledFilter CompiledFilter;
	CompileFilter(InPath, InFilter, CompiledFilter);

	EnumerateItemsMatchingFilter(CompiledFilter, InSink);
}

TArray<FContentBrowserItem> UContentBrowserDataSubsystem::GetItemsUnderPath(const FName InPath, const FContentBrowserDataFilter& InFilter) const
{
	TMap<FContentBrowserItemKey, FContentBrowserItem> FoundItems;
	EnumerateItemsUnderPath(InPath, InFilter, [&FoundItems](FContentBrowserItemData&& InItemData)
	{
		checkf(InItemData.IsValid(), TEXT("Enumerated items must be valid!"));

		const FContentBrowserItemKey ItemKey(InItemData);
		if (FContentBrowserItem* FoundItem = FoundItems.Find(ItemKey))
		{
			FoundItem->Append(InItemData);
		}
		else
		{
			FoundItems.Add(ItemKey, FContentBrowserItem(MoveTemp(InItemData)));
		}

		return true;
	});

	TArray<FContentBrowserItem> FoundItemsArray;
	FoundItems.GenerateValueArray(FoundItemsArray);
	FoundItemsArray.Sort([](const FContentBrowserItem& ItemOne, const FContentBrowserItem& ItemTwo)
	{
		return ItemOne.GetPrimaryInternalItem()->GetVirtualPath().Compare(ItemTwo.GetPrimaryInternalItem()->GetVirtualPath()) < 0;
	});
	return FoundItemsArray;
}

void UContentBrowserDataSubsystem::EnumerateItemsAtPath(const FName InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItem&&)> InCallback) const
{
	EnumerateItemsAtPath(InPath, InItemTypeFilter, [&InCallback](FContentBrowserItemData&& InItemData)
	{
		checkf(InItemData.IsValid(), TEXT("Enumerated items must be valid!"));
		return InCallback(FContentBrowserItem(MoveTemp(InItemData)));
	});
}

void UContentBrowserDataSubsystem::EnumerateItemsAtPath(const FName InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) const
{
	bool bHandledVirtualFolder = false;
	for (const TPair<FName, UContentBrowserDataSource*>& ActiveDataSourcePair : ActiveDataSources)
	{
		UContentBrowserDataSource* DataSource = ActiveDataSourcePair.Value;
		FName InternalPath;
		const EContentBrowserPathType ConvertedPathType = DataSource->TryConvertVirtualPath(InPath, InternalPath);
		if (ConvertedPathType == EContentBrowserPathType::Internal)
		{
			DataSource->EnumerateItemsAtPath(InPath, InItemTypeFilter, InCallback);
		}
		else if (ConvertedPathType == EContentBrowserPathType::Virtual)
		{
			if (!bHandledVirtualFolder && EnumHasAnyFlags(InItemTypeFilter, EContentBrowserItemTypeFilter::IncludeFolders))
			{
				InCallback(DataSource->CreateVirtualFolderItem(InPath));
				bHandledVirtualFolder = true;
			}
		}
	}
}

bool UContentBrowserDataSubsystem::EnumerateItemsAtPaths(const TArrayView<struct FContentBrowserItemPath> InItemPaths, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) const
{
	for (const auto& ActiveDataSourcePair : ActiveDataSources)
	{
		UContentBrowserDataSource* DataSource = ActiveDataSourcePair.Value;
		if (!DataSource->EnumerateItemsAtPaths(InItemPaths, InItemTypeFilter, InCallback))
		{
			return false;
		}
	}

	return true;
}

void UContentBrowserDataSubsystem::EnumerateItemsAtUserProvidedPath(const FStringView InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItem&&)> InCallback) const
{
	EnumerateItemsAtUserProvidedPath(InPath, InItemTypeFilter, [&InCallback](FContentBrowserItemData&& InItemData)
	{
		checkf(InItemData.IsValid(), TEXT("Enumerated items must be valid!"));
		return InCallback(FContentBrowserItem(MoveTemp(InItemData)));
	});
}

void UContentBrowserDataSubsystem::EnumerateItemsAtUserProvidedPath(const FStringView InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) const
{
	for (const TPair<FName, UContentBrowserDataSource*>& ActiveDataSourcePair : ActiveDataSources)
	{
		UContentBrowserDataSource* DataSource = ActiveDataSourcePair.Value;
		DataSource->EnumerateItemsAtUserProvidedPath(InPath, InItemTypeFilter, InCallback);
	}
}

bool UContentBrowserDataSubsystem::EnumerateItemsForObjects(TArrayView<UObject*> InObjects, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) const
{
	for (const auto& ActiveDataSourcePair : ActiveDataSources)
	{
		UContentBrowserDataSource* DataSource = ActiveDataSourcePair.Value;
		if (!DataSource->EnumerateItemsForObjects(InObjects, InCallback))
		{
			return false;
		}
	}

	return true;
}

TArray<FContentBrowserItem> UContentBrowserDataSubsystem::GetItemsAtPath(const FName InPath, const EContentBrowserItemTypeFilter InItemTypeFilter) const
{
	TMap<FContentBrowserItemKey, FContentBrowserItem> FoundItems;
	EnumerateItemsAtPath(InPath, InItemTypeFilter, [&FoundItems](FContentBrowserItemData&& InItemData)
	{
		checkf(InItemData.IsValid(), TEXT("Enumerated items must be valid!"));

		FContentBrowserItem& FoundItem = FoundItems.FindOrAdd(FContentBrowserItemKey(InItemData));
		FoundItem.Append(InItemData);

		return true;
	});

	TArray<FContentBrowserItem> FoundItemsArray;
	FoundItems.GenerateValueArray(FoundItemsArray);
	return FoundItemsArray;
}

FContentBrowserItem UContentBrowserDataSubsystem::GetItemAtPath(const FName InPath, const EContentBrowserItemTypeFilter InItemTypeFilter) const
{
	FContentBrowserItem FoundItem;
	EnumerateItemsAtPath(InPath, InItemTypeFilter, [&FoundItem](FContentBrowserItemData&& InItemData)
	{
		checkf(InItemData.IsValid(), TEXT("Enumerated items must be valid!"));

		if (FoundItem.IsValid())
		{
			if (FContentBrowserItemKey(FoundItem) == FContentBrowserItemKey(InItemData))
			{
				FoundItem.Append(InItemData);
			}
		}
		else
		{
			FoundItem = FContentBrowserItem(MoveTemp(InItemData));
		}

		return true;
	});
	return FoundItem;
}

TArray<FContentBrowserItem> UContentBrowserDataSubsystem::GetItemsAtUserProvidedPath(const FStringView InPath, const EContentBrowserItemTypeFilter InItemTypeFilter) const
{
	TMap<FContentBrowserItemKey, FContentBrowserItem> FoundItems;
	EnumerateItemsAtUserProvidedPath(InPath, InItemTypeFilter, [&FoundItems](FContentBrowserItemData&& InItemData)
	{
		checkf(InItemData.IsValid(), TEXT("Enumerated items must be valid!"));

		FContentBrowserItem& FoundItem = FoundItems.FindOrAdd(FContentBrowserItemKey(InItemData));
		FoundItem.Append(InItemData);

		return true;
	});

	TArray<FContentBrowserItem> FoundItemsArray;
	FoundItems.GenerateValueArray(FoundItemsArray);
	return FoundItemsArray;
}

TArray<FContentBrowserItemPath> UContentBrowserDataSubsystem::GetAliasesForPath(const FContentBrowserItemPath InPath) const
{
	return GetAliasesForPath(InPath.GetInternalPathName());
}

TArray<FContentBrowserItemPath> UContentBrowserDataSubsystem::GetAliasesForPath(const FSoftObjectPath& InInternalPath) const
{
	TArray<FContentBrowserItemPath> Aliases;

	for (const auto& ActiveDataSourcePair : ActiveDataSources)
	{
		UContentBrowserDataSource* DataSource = ActiveDataSourcePair.Value;
		Aliases.Append(DataSource->GetAliasesForPath(InInternalPath));
	}

	return Aliases;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TArray<FContentBrowserItemPath> UContentBrowserDataSubsystem::GetAliasesForPath(const FName InInternalPath) const
{
	TArray<FContentBrowserItemPath> Aliases;

	for (const auto& ActiveDataSourcePair : ActiveDataSources)
	{
		UContentBrowserDataSource* DataSource = ActiveDataSourcePair.Value;
		Aliases.Append(DataSource->GetAliasesForPath(InInternalPath));
	}

	return Aliases;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool UContentBrowserDataSubsystem::IsDiscoveringItems(TArray<FText>* OutStatus) const
{
	bool bIsDiscoveringItems = false;
	for (const auto& ActiveDataSourcePair : ActiveDataSources)
	{
		FText DataSourceStatus;
		if (ActiveDataSourcePair.Value->IsDiscoveringItems(&DataSourceStatus))
		{
			bIsDiscoveringItems = true;
			if (OutStatus && !DataSourceStatus.IsEmpty())
			{
				OutStatus->Emplace(MoveTemp(DataSourceStatus));
			}
		}
	}
	return bIsDiscoveringItems;
}

bool UContentBrowserDataSubsystem::PrioritizeSearchPath(const FName InPath)
{
	bool bDidPrioritize = false;
	for (const auto& ActiveDataSourcePair : ActiveDataSources)
	{
		UContentBrowserDataSource* DataSource = ActiveDataSourcePair.Value;
		if (DataSource->IsVirtualPathUnderMountRoot(InPath))
		{
			bDidPrioritize |= DataSource->PrioritizeSearchPath(InPath);
		}
	}
	return bDidPrioritize;
}

bool UContentBrowserDataSubsystem::IsFolderVisible(const FName InPath, const EContentBrowserIsFolderVisibleFlags InFlags) const
{
	FContentBrowserFolderContentsFilter ContentsFilter;
	ContentsFilter.HideFolderIfEmptyFilter = CreateHideFolderIfEmptyFilter();

	return IsFolderVisible(InPath, InFlags, ContentsFilter);
}

bool UContentBrowserDataSubsystem::IsFolderVisible(const FName InPath, const EContentBrowserIsFolderVisibleFlags InFlags, const FContentBrowserFolderContentsFilter& InContentsFilter) const
{
	bool bIsKnownPath = false;
	for (const auto& ActiveDataSourcePair : ActiveDataSources)
	{
		UContentBrowserDataSource* DataSource = ActiveDataSourcePair.Value;
		if (DataSource->IsVirtualPathUnderMountRoot(InPath))
		{
			bIsKnownPath = true;
			if (DataSource->IsFolderVisible(InPath, InFlags, InContentsFilter))
			{
				return true;
			}
		}
	}

	// Return true if this is visible for any sources, or this path isn't handled by any of the sources
	return !bIsKnownPath;
}

bool UContentBrowserDataSubsystem::IsFolderVisibleIfHidingEmpty(const FName InPath) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return IsFolderVisible(InPath, EContentBrowserIsFolderVisibleFlags::Default | EContentBrowserIsFolderVisibleFlags::HideEmptyFolders);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UContentBrowserDataSubsystem::CanCreateFolder(const FName InPath, FText* OutErrorMsg) const
{
	for (const auto& ActiveDataSourcePair : ActiveDataSources)
	{
		UContentBrowserDataSource* DataSource = ActiveDataSourcePair.Value;
		if (DataSource->IsVirtualPathUnderMountRoot(InPath))
		{
			if (DataSource->CanCreateFolder(InPath, OutErrorMsg))
			{
				return true;
			}
		}
	}
	return false;
}

FContentBrowserItemTemporaryContext UContentBrowserDataSubsystem::CreateFolder(const FName InPath) const
{
	return CreateFolder(InPath, CreateHideFolderIfEmptyFilter());
}

FContentBrowserItemTemporaryContext UContentBrowserDataSubsystem::CreateFolder(const FName InPath, const TSharedPtr<IContentBrowserHideFolderIfEmptyFilter>& HideFolderIfEmptyFilter) const
{
	FContentBrowserItemTemporaryContext NewItem;
	for (const auto& ActiveDataSourcePair : ActiveDataSources)
	{
		UContentBrowserDataSource* DataSource = ActiveDataSourcePair.Value;
		if (DataSource->IsVirtualPathUnderMountRoot(InPath))
		{
			FContentBrowserItemDataTemporaryContext NewItemData;
			if (DataSource->CreateFolder(InPath, HideFolderIfEmptyFilter, NewItemData))
			{
				NewItem.AppendContext(MoveTemp(NewItemData));
			}
		}
	}
	return NewItem;
}

void UContentBrowserDataSubsystem::Legacy_TryConvertPackagePathToVirtualPaths(const FName InPackagePath, TFunctionRef<bool(FName)> InCallback)
{
	for (const auto& ActiveDataSourcePair : ActiveDataSources)
	{
		UContentBrowserDataSource* DataSource = ActiveDataSourcePair.Value;

		FName VirtualPath;
		if (DataSource->Legacy_TryConvertPackagePathToVirtualPath(InPackagePath, VirtualPath))
		{
			if (!InCallback(VirtualPath))
			{
				break;
			}
		}
	}
}

void UContentBrowserDataSubsystem::Legacy_TryConvertAssetDataToVirtualPaths(const FAssetData& InAssetData, const bool InUseFolderPaths, TFunctionRef<bool(FName)> InCallback)
{
	for (const auto& ActiveDataSourcePair : ActiveDataSources)
	{
		UContentBrowserDataSource* DataSource = ActiveDataSourcePair.Value;

		FName VirtualPath;
		if (DataSource->Legacy_TryConvertAssetDataToVirtualPath(InAssetData, InUseFolderPaths, VirtualPath))
		{
			if (!InCallback(VirtualPath))
			{
				break;
			}
		}
	}
}

void UContentBrowserDataSubsystem::RefreshVirtualPathTreeIfNeeded()
{
	for (const auto& ActiveDataSourcePair : ActiveDataSources)
	{
		UContentBrowserDataSource* DataSource = ActiveDataSourcePair.Value;
		DataSource->RefreshVirtualPathTreeIfNeeded();
	}
}

void UContentBrowserDataSubsystem::SetVirtualPathTreeNeedsRebuild()
{
	for (const auto& ActiveDataSourcePair : ActiveDataSources)
	{
		UContentBrowserDataSource* DataSource = ActiveDataSourcePair.Value;
		DataSource->SetVirtualPathTreeNeedsRebuild();
	}
}


void UContentBrowserDataSubsystem::FContentBrowserFilterCacheApi::InitializeCacheIDOwner(UContentBrowserDataSubsystem& Subsystem, FContentBrowserDataFilterCacheIDOwner& IDOwner)
{
	Subsystem.InitializeCacheIDOwner(IDOwner);
}

void UContentBrowserDataSubsystem::FContentBrowserFilterCacheApi::RemoveUnusedCachedData(const UContentBrowserDataSubsystem& Subsystem, const FContentBrowserDataFilterCacheIDOwner& IDOwner, TArrayView<const FName> InVirtualPathsInUse, const FContentBrowserDataFilter& DataFilter)
{
	Subsystem.RemoveUnusedCachedFilterData(IDOwner, InVirtualPathsInUse, DataFilter);
}

void UContentBrowserDataSubsystem::FContentBrowserFilterCacheApi::ClearCachedData(const UContentBrowserDataSubsystem& Subsystem, const FContentBrowserDataFilterCacheIDOwner& IDOwner)
{
	Subsystem.ClearCachedFilterData(IDOwner);
}


void UContentBrowserDataSubsystem::InitializeCacheIDOwner(FContentBrowserDataFilterCacheIDOwner& IDOwner)
{
	++LastCacheIDForFilter;
	if (LastCacheIDForFilter == INDEX_NONE)
	{
		++LastCacheIDForFilter;
	}

	IDOwner.ID = LastCacheIDForFilter;
	IDOwner.DataSource = this;
}

void UContentBrowserDataSubsystem::RemoveUnusedCachedFilterData(const FContentBrowserDataFilterCacheIDOwner& IDOwner, TArrayView<const FName> InVirtualPathsInUse, const FContentBrowserDataFilter& DataFilter) const 
{
	for (const TPair<FName, UContentBrowserDataSource*>& DataSource : AvailableDataSources)
	{
		DataSource.Value->RemoveUnusedCachedFilterData(IDOwner, InVirtualPathsInUse, DataFilter);
	}
}

void UContentBrowserDataSubsystem::ClearCachedFilterData(const FContentBrowserDataFilterCacheIDOwner& IDOwner) const
{
	for (const TPair<FName, UContentBrowserDataSource*>& DataSource : AvailableDataSources)
	{
		DataSource.Value->ClearCachedFilterData(IDOwner);
	}
}

void UContentBrowserDataSubsystem::HandleDataSourceRegistered(const FName& Type, IModularFeature* Feature)
{
	if (Type == UContentBrowserDataSource::GetModularFeatureTypeName())
	{
		UContentBrowserDataSource* DataSource = static_cast<UContentBrowserDataSource*>(Feature);

		checkf(DataSource->IsInitialized(), TEXT("Data source '%s' was uninitialized! Did you forget to call Initialize?"), *DataSource->GetName());

		AvailableDataSources.Add(DataSource->GetFName(), DataSource);

		if (EnabledDataSources.Contains(DataSource->GetFName()))
		{
			ActivateDataSource(DataSource->GetFName());
		}
	}
}

void UContentBrowserDataSubsystem::HandleDataSourceUnregistered(const FName& Type, IModularFeature* Feature)
{
	if (Type == UContentBrowserDataSource::GetModularFeatureTypeName())
	{
		UContentBrowserDataSource* DataSource = static_cast<UContentBrowserDataSource*>(Feature);

		if (AvailableDataSources.Contains(DataSource->GetFName()))
		{
			DeactivateDataSource(DataSource->GetFName());
		}

		AvailableDataSources.Remove(DataSource->GetFName());
	}
}

void UContentBrowserDataSubsystem::Tick(const float InDeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UContentBrowserDataSubsystem_Tick);

	if (UE::IsSavingPackage() || IsGarbageCollecting() || FUObjectThreadContext::Get().IsRoutingPostLoad)
	{
		// Not to safe to Tick right now, as the below code may try and find objects
		return;
	}

	if (TickSuppressionCount > 0)
	{
		// Not safe to Tick right now, as we've been asked not to
		return;
	}

	if (bContentMountedThisFrame)
	{
		// Content just added, defer tick for a frame or we risk slowing down content load
		bContentMountedThisFrame = false;
		return;
	}

	for (const auto& AvailableDataSourcePair : AvailableDataSources)
	{
		AvailableDataSourcePair.Value->Tick(InDeltaTime);
	}

	if (bPendingItemDataRefreshedNotification)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UContentBrowserDataSubsystem::BroadcastItemDataRefreshed);

		bPendingItemDataRefreshedNotification = false;
		DelayedPendingUpdates.Empty();
		PendingUpdates.Empty();
		ItemDataRefreshedDelegate.Broadcast();
	}

	if (PendingUpdates.Num() > 0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UContentBrowserDataSubsystem::BroadCastItemDataUpdate);

		TArray<FContentBrowserItemDataUpdate> LocalPendingUpdates = MoveTemp(PendingUpdates);
		PendingUpdates.Empty();
		ItemDataUpdatedDelegate.Broadcast(MakeArrayView(LocalPendingUpdates));
	}

	if (ActiveDataSourcesDiscoveringContent.Num() > 0)
	{
		for (auto It = ActiveDataSourcesDiscoveringContent.CreateIterator(); It; ++It)
		{
			if (UContentBrowserDataSource* DataSource = ActiveDataSources.FindRef(*It))
			{
				// Has this source finished its content discovery?
				if (!DataSource->IsDiscoveringItems())
				{
					It.RemoveCurrent();
					continue;
				}
			}
			else
			{
				// Source no longer active - just remove this entry
				It.RemoveCurrent();
				continue;
			}
		}

		if (ActiveDataSourcesDiscoveringContent.Num() == 0)
		{
			ItemDataDiscoveryCompleteDelegate.Broadcast();
		}
	}
}

void UContentBrowserDataSubsystem::OnContentPathMounted(const FString& AssetPath, const FString& ContentPath)
{
	bContentMountedThisFrame = true;
}

void UContentBrowserDataSubsystem::QueueItemDataUpdate(FContentBrowserItemDataUpdate&& InUpdate)
{
	if (!AllowModifiedItemDataUpdates())
	{
		const EContentBrowserItemUpdateType UpdateType = InUpdate.GetUpdateType();

		// Ignore modified during PIE to reduce hitches, they will be queue and then added to the pending updates when PIE stops
		if (UpdateType == EContentBrowserItemUpdateType::Modified)
		{
			FContentBrowserItemKey ItemKey(InUpdate.GetItemData());
			DelayedPendingUpdates.Add(MoveTemp(ItemKey), MoveTemp(InUpdate));
		}
		else
		{
			// Clear the delayed update for the item if there was one 
			if (UpdateType == EContentBrowserItemUpdateType::Moved)
			{
				const FContentBrowserItemData& ItemData = InUpdate.GetItemData();
				FContentBrowserItemKey ItemKey(ItemData.GetItemType(), InUpdate.GetPreviousVirtualPath(), ItemData.GetOwnerDataSource());
				DelayedPendingUpdates.Remove(ItemKey);
			}
			else
			{
				FContentBrowserItemKey ItemKey(InUpdate.GetItemData());
				DelayedPendingUpdates.Remove(ItemKey);
			}

			// TODO: Merge multiple Modified updates for a single item?
			PendingUpdates.Emplace(MoveTemp(InUpdate));
		}
	}
	else
	{
		// TODO: Merge multiple Modified updates for a single item?
		PendingUpdates.Emplace(MoveTemp(InUpdate));
	}
}

void UContentBrowserDataSubsystem::NotifyItemDataRefreshed()
{
	bPendingItemDataRefreshedNotification = true;
}

bool UContentBrowserDataSubsystem::AllowModifiedItemDataUpdates() const
{
	return !bIsPIEActive;
}

void UContentBrowserDataSubsystem::OnBeginPIE(const bool bIsSimulating)
{
	bIsPIEActive = true;
}

void UContentBrowserDataSubsystem::OnEndPIE(const bool bIsSimulating)
{
	bIsPIEActive = false;

	if (!DelayedPendingUpdates.IsEmpty())
	{
		// Move the DelayedPendingUpdates into the PendingUpdates.
		PendingUpdates.Reserve(DelayedPendingUpdates.Num() + PendingUpdates.Num());
		
		for (TPair<FContentBrowserItemKey, FContentBrowserItemDataUpdate>& Pair : DelayedPendingUpdates)
		{
			PendingUpdates.Add(MoveTemp(Pair.Value));
		}

		DelayedPendingUpdates.Empty();
	}
}

void UContentBrowserDataSubsystem::SetPathViewSpecialSortFolders(const TArray<FName>& InSpecialSortFolders)
{
	PathViewSpecialSortFolders = InSpecialSortFolders;
}

const TArray<FName>& UContentBrowserDataSubsystem::GetDefaultPathViewSpecialSortFolders() const
{
	return DefaultPathViewSpecialSortFolders;
}

const TArray<FName>& UContentBrowserDataSubsystem::GetPathViewSpecialSortFolders() const
{
	return PathViewSpecialSortFolders;
}

void UContentBrowserDataSubsystem::ConvertInternalPathToVirtual(const FStringView InPath, FStringBuilderBase& OutPath)
{
	OutPath.Reset();

	if (GetDefault<UContentBrowserSettings>()->bShowAllFolder)
	{
		OutPath.Append(AllFolderPrefix);
		if (InPath.Len() == 1 && InPath.Equals(TEXT("/")))
		{
			return;
		}
	}

	TOptional<FStringView> MountPointOptional;
	auto GetMountPoint = [&MountPointOptional, InPath]()
	{
		if (!MountPointOptional.IsSet())
		{
			MountPointOptional = FPathViews::GetMountPointNameFromPath(InPath);
		}
		return MountPointOptional.GetValue();
	};

	TOptional<TSharedPtr<IPlugin>> PluginOptional;
	auto GetPlugin = [&PluginOptional, GetMountPoint]()
	{
		if (!PluginOptional.IsSet())
		{
			PluginOptional = IPluginManager::Get().FindPlugin(GetMountPoint());
		}
		return PluginOptional.GetValue();
	};

	if (UsePluginVersePathDelegate.IsBound())
	{
		if (const TSharedPtr<IPlugin> Plugin = GetPlugin())
		{
			if (UsePluginVersePath(Plugin.ToSharedRef()))
			{
				OutPath.Append(FPathViews::GetPath(UE::String::RemoveFromEnd(FStringView(Plugin->GetVersePath()), TEXTVIEW("/"))));
				OutPath.Append(InPath.GetData(), InPath.Len());
				return;
			}
		}
	}

	if (GetDefault<UContentBrowserSettings>()->bOrganizeFolders && InPath.Len() > 1)
	{
		if (GenerateVirtualPathPrefixDelegate.IsBound())
		{
			GenerateVirtualPathPrefixDelegate.Execute(InPath, OutPath);
		}
		else
		{
			if (TSharedPtr<IPlugin> Plugin = GetPlugin())
			{
				if (Plugin->GetLoadedFrom() == EPluginLoadedFrom::Engine)
				{
					OutPath.Append(TEXT("/EngineData/Plugins"));
				}
				else
				{
					OutPath.Append(TEXT("/Plugins"));
				}

				const FPluginDescriptor& PluginDescriptor = Plugin->GetDescriptor();
				if (!PluginDescriptor.EditorCustomVirtualPath.IsEmpty())
				{
					int32 NumChars = PluginDescriptor.EditorCustomVirtualPath.Len();
					if (PluginDescriptor.EditorCustomVirtualPath.EndsWith(TEXT("/")))
					{
						--NumChars;
					}

					if (NumChars > 0)
					{
						if (!PluginDescriptor.EditorCustomVirtualPath.StartsWith(TEXT("/")))
						{
							OutPath.Append(TEXT("/"));
						}

						OutPath.Append(*PluginDescriptor.EditorCustomVirtualPath, NumChars);
					}
				}
			}
			else if (GetMountPoint().Equals(TEXT("Engine")))
			{
				OutPath.Append(TEXT("/EngineData"));
			}
		}
	}
	
	OutPath.Append(InPath.GetData(), InPath.Len());
}

void UContentBrowserDataSubsystem::ConvertInternalPathToVirtual(const FStringView InPath, FName& OutPath)
{
	FNameBuilder PathBuffer;
	ConvertInternalPathToVirtual(InPath, PathBuffer);
	OutPath = FName(PathBuffer);
}

void UContentBrowserDataSubsystem::ConvertInternalPathToVirtual(FName InPath, FName& OutPath)
{
	ConvertInternalPathToVirtual(FNameBuilder(InPath), OutPath);
}

FName UContentBrowserDataSubsystem::ConvertInternalPathToVirtual(FName InPath)
{
	FName OutPath;
	ConvertInternalPathToVirtual(InPath, OutPath);
	return OutPath;
}

TArray<FString> UContentBrowserDataSubsystem::ConvertInternalPathsToVirtual(const TArray<FString>& InPaths)
{
	TArray<FString> VirtualPaths;
	VirtualPaths.Reserve(InPaths.Num());

	FNameBuilder Builder;
	for (const FString& It : InPaths)
	{
		ConvertInternalPathToVirtual(It, Builder);
		VirtualPaths.Add(FString(Builder));
	}

	return VirtualPaths;
}

FText UContentBrowserDataSubsystem::ConvertVirtualPathToDisplay(const FStringView InVirtualPath, const EContentBrowserItemTypeFilter InLeafType) const
{
	FString DisplayPath;
	DisplayPath.Reserve(InVirtualPath.Len());

	TArray<FStringView, TInlineAllocator<8>> Components;
	FPathViews::IterateComponents(InVirtualPath, [&Components](FStringView Component)
	{
		if (!Component.IsEmpty())
		{
			Components.Add(Component);
		}
	});

	FNameBuilder PathBuilder;
	for (int32 Index = 0; Index < Components.Num(); ++Index)
	{
		PathBuilder.AppendChar(TEXT('/'));
		PathBuilder.Append(Components[Index]);

		DisplayPath += TEXT('/');
		const FContentBrowserItem Item = GetItemAtPath(FName(PathBuilder), Index + 1 == Components.Num() ? InLeafType : EContentBrowserItemTypeFilter::IncludeFolders);
		if (Item.IsValid())
		{
			DisplayPath += Item.GetDisplayName().ToString();
		}
		else
		{
			DisplayPath += Components[Index];
		}
	}

	return FText::FromString(MoveTemp(DisplayPath));
}

FText UContentBrowserDataSubsystem::ConvertVirtualPathToDisplay(FName InVirtualPath, const EContentBrowserItemTypeFilter InLeafType) const
{
	FNameBuilder VirtualPath;
	InVirtualPath.AppendString(VirtualPath);
	return ConvertVirtualPathToDisplay(VirtualPath.ToView(), InLeafType);
}

FText UContentBrowserDataSubsystem::ConvertVirtualPathToDisplay(const FContentBrowserItem& InItem) const
{
	FNameBuilder VirtualPath;
	InItem.GetVirtualPath().AppendString(VirtualPath);

	FString DisplayPath;
	DisplayPath.Reserve(VirtualPath.Len());

	TArray<FStringView, TInlineAllocator<8>> Components;
	FPathViews::IterateComponents(VirtualPath, [&Components](FStringView Component)
	{
		if (!Component.IsEmpty())
		{
			Components.Add(Component);
		}
	});

	FNameBuilder PathBuilder;
	for (int32 Index = 0; Index < Components.Num(); ++Index)
	{
		PathBuilder.AppendChar(TEXT('/'));
		PathBuilder.Append(Components[Index]);

		DisplayPath += TEXT('/');
		if (Index + 1 == Components.Num())
		{
			DisplayPath += InItem.GetDisplayName().ToString();
		}
		else
		{
			const FContentBrowserItem Item = GetItemAtPath(FName(PathBuilder), EContentBrowserItemTypeFilter::IncludeFolders);
			if (Item.IsValid())
			{
				DisplayPath += Item.GetDisplayName().ToString();
			}
			else
			{
				DisplayPath += Components[Index];
			}
		}
	}

	return FText::FromString(MoveTemp(DisplayPath));
}

void UContentBrowserDataSubsystem::SetGenerateVirtualPathPrefixDelegate(const FContentBrowserGenerateVirtualPathDelegate& InDelegate)
{
	GenerateVirtualPathPrefixDelegate = InDelegate;
	SetVirtualPathTreeNeedsRebuild();
	RefreshVirtualPathTreeIfNeeded();
}

FContentBrowserGenerateVirtualPathDelegate& UContentBrowserDataSubsystem::OnGenerateVirtualPathPrefix()
{
	return GenerateVirtualPathPrefixDelegate;
}

bool UContentBrowserDataSubsystem::UsePluginVersePath(const TSharedRef<IPlugin>& Plugin)
{
	if (UsePluginVersePathDelegate.IsBound() && !Plugin->GetVersePath().IsEmpty())
	{
		return UsePluginVersePathDelegate.Execute(Plugin);
	}
	return false;
}

void UContentBrowserDataSubsystem::SetUsePluginVersePathDelegate(FContentBrowserUsePluginVersePathDelegate InDelegate)
{
	UsePluginVersePathDelegate = MoveTemp(InDelegate);
	SetVirtualPathTreeNeedsRebuild();
	RefreshVirtualPathTreeIfNeeded();
}

FContentBrowserUsePluginVersePathDelegate& UContentBrowserDataSubsystem::GetUsePluginVersePathDelegate()
{
	return UsePluginVersePathDelegate;
}

TSharedPtr<IContentBrowserHideFolderIfEmptyFilter> UContentBrowserDataSubsystem::CreateHideFolderIfEmptyFilter() const
{
	if (CreateHideFolderIfEmptyFilterDelegates.IsEmpty())
	{
		return DefaultHideFolderIfEmptyFilter;
	}

	TArray<TSharedPtr<IContentBrowserHideFolderIfEmptyFilter>> HideFolderIfEmptyFilters;
	HideFolderIfEmptyFilters.Reserve(1 + CreateHideFolderIfEmptyFilterDelegates.Num());

	if (DefaultHideFolderIfEmptyFilter)
	{
		HideFolderIfEmptyFilters.Add(DefaultHideFolderIfEmptyFilter);
	}

	for (const FContentBrowserCreateHideFolderIfEmptyFilter& CreateHideFolderIfEmptyFilter : CreateHideFolderIfEmptyFilterDelegates)
	{
		TSharedPtr<IContentBrowserHideFolderIfEmptyFilter> HideFolderIfEmptyFilter = CreateHideFolderIfEmptyFilter.Execute();
		if (HideFolderIfEmptyFilter.IsValid())
		{
			HideFolderIfEmptyFilters.Add(MoveTemp(HideFolderIfEmptyFilter));
		}
	}

	if (HideFolderIfEmptyFilters.Num() == 1)
	{
		return HideFolderIfEmptyFilters[0];
	}

	return MakeShared<ContentBrowserDataSubsystem::FMergedHideFolderIfEmptyFilter>(MoveTemp(HideFolderIfEmptyFilters));
}

FDelegateHandle UContentBrowserDataSubsystem::RegisterCreateHideFolderIfEmptyFilter(FContentBrowserCreateHideFolderIfEmptyFilter Delegate)
{
	return CreateHideFolderIfEmptyFilterDelegates.Add_GetRef(MoveTemp(Delegate)).GetHandle();
}

void UContentBrowserDataSubsystem::UnregisterCreateHideFolderIfEmptyFilter(FDelegateHandle DelegateHandle)
{
	int32 Index = CreateHideFolderIfEmptyFilterDelegates.IndexOfByPredicate([DelegateHandle](const FContentBrowserCreateHideFolderIfEmptyFilter& Delegate)
		{
			return DelegateHandle == Delegate.GetHandle();
		});
	if (Index != INDEX_NONE)
	{
		CreateHideFolderIfEmptyFilterDelegates.RemoveAtSwap(Index);
	}
}

const FString& UContentBrowserDataSubsystem::GetAllFolderPrefix() const
{
	return AllFolderPrefix;
}

TSharedRef<FPathPermissionList>& UContentBrowserDataSubsystem::GetEditableFolderPermissionList()
{
	return EditableFolderPermissionList;
}

EContentBrowserPathType UContentBrowserDataSubsystem::TryConvertVirtualPath(const FStringView InPath, FStringBuilderBase& OutPath) const
{
	FNameBuilder FoundVirtualPath;
	for (const auto& ActiveDataSourcePair : ActiveDataSources)
	{
		UContentBrowserDataSource* DataSource = ActiveDataSourcePair.Value;
		const EContentBrowserPathType PathType = DataSource->TryConvertVirtualPath(InPath, OutPath);
		if (PathType != EContentBrowserPathType::None)
		{
			if (PathType == EContentBrowserPathType::Internal)
			{
				return PathType;
			}
			else if (PathType == EContentBrowserPathType::Virtual)
			{
				// Another data source may be able to convert this to internal so keep checking
				// Only after all data sources had a chance to claim ownership (internal) do we return 
				// Example: /Classes_Game is known to classes data source but not to asset data source
				FoundVirtualPath.Reset();
				FoundVirtualPath.Append(OutPath);
			}
		}
	}

	if (FoundVirtualPath.Len() > 0)
	{
		OutPath.Reset();
		OutPath.Append(FoundVirtualPath);
		return EContentBrowserPathType::Virtual;
	}
	else
	{
		return EContentBrowserPathType::None;
	}
}

EContentBrowserPathType UContentBrowserDataSubsystem::TryConvertVirtualPath(FStringView InPath, FString& OutPath) const
{
	FNameBuilder OutPathBuilder;
	const EContentBrowserPathType ConvertedType = TryConvertVirtualPath(InPath, OutPathBuilder);
	OutPath = FString(FStringView(OutPathBuilder));
	return ConvertedType;
}

EContentBrowserPathType UContentBrowserDataSubsystem::TryConvertVirtualPath(FStringView InPath, FName& OutPath) const
{
	FNameBuilder OutPathBuilder;
	const EContentBrowserPathType ConvertedType = TryConvertVirtualPath(InPath, OutPathBuilder);
	OutPath = FName(FStringView(OutPathBuilder));
	return ConvertedType;
}

EContentBrowserPathType UContentBrowserDataSubsystem::TryConvertVirtualPath(FName InPath, FName& OutPath) const
{
	FNameBuilder OutPathBuilder;
	const EContentBrowserPathType ConvertedType = TryConvertVirtualPath(FNameBuilder(InPath), OutPathBuilder);
	OutPath = FName(FStringView(OutPathBuilder));
	return ConvertedType;
}

TArray<FString> UContentBrowserDataSubsystem::TryConvertVirtualPathsToInternal(const TArray<FString>& InPaths) const
{
	TArray<FString> InternalPaths;
	InternalPaths.Reserve(InPaths.Num());

	for (const FString& VirtualPath : InPaths)
	{
		FString ConvertedPath;
		if (TryConvertVirtualPath(VirtualPath, ConvertedPath) == EContentBrowserPathType::Internal)
		{
			InternalPaths.Add(MoveTemp(ConvertedPath));
		}
	}

	return InternalPaths;
}
