// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserDataUtils.h"

#include "AssetRegistry/AssetData.h"
#include "Containers/StringView.h"
#include "ContentBrowserDataSubsystem.h"
#include "Editor.h"
#include "IContentBrowserDataModule.h"
#include "Interfaces/IPluginManager.h"
#include "Internationalization/Internationalization.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/PackageName.h"
#include "Misc/PackagePath.h"
#include "Misc/Paths.h"
#include "Misc/StringBuilder.h"
#include "Settings/ContentBrowserSettings.h"
#include "Templates/SharedPointer.h"
#include "UObject/UObjectGlobals.h"
#include "VerseVM/VVMVerse.h"
#include "VerseVM/VVMVerseClass.h"

#define LOCTEXT_NAMESPACE "ContentBrowserAssetDataUtils"

int32 ContentBrowserDataUtils::CalculateFolderDepthOfPath(const FStringView InPath)
{
	int32 Depth = 0;
	if (InPath.Len() > 1)
	{
		++Depth;

		// Ignore first and final characters
		const TCHAR* Current = InPath.GetData() + 1;
		const TCHAR* End = InPath.GetData() + InPath.Len() - 1;
		for (; Current != End; ++Current)
		{
			if (*Current == TEXT('/'))
			{
				++Depth;
			}
		}
	}

	return Depth;
}

bool ContentBrowserDataUtils::IsTopLevelFolder(const FStringView InFolderPath)
{
	int32 SlashCount = 0;
	for (const TCHAR PathChar : InFolderPath)
	{
		if (PathChar == TEXT('/'))
		{
			if (++SlashCount > 1)
			{
				break;
			}
		}
	}

	return SlashCount == 1;
}

bool ContentBrowserDataUtils::IsTopLevelFolder(const FName InFolderPath)
{
	return IsTopLevelFolder(FNameBuilder(InFolderPath));
}

int32 ContentBrowserDataUtils::GetMaxFolderDepthRequiredForAttributeFilter()
{
	static const int32 MaxFolderDepthToCheck = FMath::Max(
			ContentBrowserDataUtils::CalculateFolderDepthOfPath(FPackageName::FilenameToLongPackageName(FPaths::GameDevelopersDir()).LeftChop(1))
		, 2);

	return MaxFolderDepthToCheck;
}
 
bool ContentBrowserDataUtils::PathPassesAttributeFilter(const FStringView InPath, const int32 InAlreadyCheckedDepth, const EContentBrowserItemAttributeFilter InAttributeFilter)
{
	// Note Keep the implementation of that function pure as it is called from outside of the game thread

	static const FString ProjectContentRootName = TEXT("Game");
	static const FString EngineContentRootName = TEXT("Engine");
	static const FString LocalizationFolderName = TEXT("L10N");
	static const FString ExternalActorsFolderName = FPackagePath::GetExternalActorsFolderName();
	static const FString ExternalObjectsFolderName = FPackagePath::GetExternalObjectsFolderName();
	static const FString DeveloperPathWithoutSlash = FPackageName::FilenameToLongPackageName(FPaths::GameDevelopersDir()).LeftChop(1);
	static const int32 DevelopersFolderDepth = ContentBrowserDataUtils::CalculateFolderDepthOfPath(DeveloperPathWithoutSlash);
	const int32 MaxFolderDepthToCheck = ContentBrowserDataUtils::GetMaxFolderDepthRequiredForAttributeFilter();

	static auto GetRootFolderNameFromPath = [](const FStringView InFullPath)
	{
		FStringView Result(InFullPath);

		// Remove '/' from start
		if (Result.StartsWith(TEXT('/')))
		{
			Result.RightChopInline(1);
		}

		// Return up until just before next '/'
		int32 FoundIndex = INDEX_NONE;
		if (Result.FindChar(TEXT('/'), FoundIndex))
		{
			Result.LeftInline(FoundIndex);
		}

		return Result;
	};

	if (InAlreadyCheckedDepth < MaxFolderDepthToCheck)
	{
		if (InAlreadyCheckedDepth < 2)
		{
			FStringView RootName = GetRootFolderNameFromPath(InPath);
			if (RootName.Len() == 0)
			{
				return true;
			}

			// if not already checked root folder
			if (InAlreadyCheckedDepth < 1)
			{
				const bool bIncludeProject = EnumHasAnyFlags(InAttributeFilter, EContentBrowserItemAttributeFilter::IncludeProject);
				const bool bIncludeEngine = EnumHasAnyFlags(InAttributeFilter, EContentBrowserItemAttributeFilter::IncludeEngine);
				const bool bIncludePlugins = EnumHasAnyFlags(InAttributeFilter, EContentBrowserItemAttributeFilter::IncludePlugins);
				if (!bIncludePlugins || !bIncludeEngine || !bIncludeProject)
				{
					if (RootName.Equals(ProjectContentRootName))
					{
						if (!bIncludeProject)
						{
							return false;
						}
					}
					else if (RootName.Equals(EngineContentRootName))
					{
						if (!bIncludeEngine)
						{
							return false;
						}
					}
					else
					{
						if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(RootName))
						{
							if (Plugin->IsEnabled() && Plugin->CanContainContent())
							{
								if (!bIncludePlugins)
								{
									return false;
								}

								const EPluginLoadedFrom PluginSource = Plugin->GetLoadedFrom();
								if (PluginSource == EPluginLoadedFrom::Engine)
								{
									if (!bIncludeEngine)
									{
										return false;
									}
								}
								else if (PluginSource == EPluginLoadedFrom::Project)
								{
									if (!bIncludeProject)
									{
										return false;
									}
								}
							}
						}
					}
				}
			}
			
			const FStringView AfterFirstFolder = InPath.RightChop(RootName.Len() + 2);
			if (AfterFirstFolder.StartsWith(ExternalActorsFolderName) && (AfterFirstFolder.Len() == ExternalActorsFolderName.Len() || AfterFirstFolder[ExternalActorsFolderName.Len()] == TEXT('/')))
			{
				return false;
			}
			if (AfterFirstFolder.StartsWith(ExternalObjectsFolderName) && (AfterFirstFolder.Len() == ExternalObjectsFolderName.Len() || AfterFirstFolder[ExternalObjectsFolderName.Len()] == TEXT('/')))
			{
				return false;
			}

			if (!EnumHasAnyFlags(InAttributeFilter, EContentBrowserItemAttributeFilter::IncludeLocalized))
			{
				if (AfterFirstFolder.StartsWith(LocalizationFolderName) && (AfterFirstFolder.Len() == LocalizationFolderName.Len() || AfterFirstFolder[LocalizationFolderName.Len()] == TEXT('/')))
				{
					return false;
				}
			}
		}

		if (InAlreadyCheckedDepth < DevelopersFolderDepth && !EnumHasAnyFlags(InAttributeFilter, EContentBrowserItemAttributeFilter::IncludeDeveloper))
		{
			if (InPath.StartsWith(DeveloperPathWithoutSlash) && (InPath.Len() == DeveloperPathWithoutSlash.Len() || InPath[DeveloperPathWithoutSlash.Len()] == TEXT('/')))
			{
				return false;
			}
		}
	}

	return true;
}

FText ContentBrowserDataUtils::GetFolderItemDisplayNameOverride(const FName InFolderPath, const FString& InFolderItemName, const bool bIsClassesFolder, const bool bIsCookedPath)
{
	if (InFolderItemName.IsEmpty())
	{
		return FText::GetEmpty();
	}

	FText FolderDisplayNameOverride;

	if (!bIsClassesFolder)
	{
		static const FName GameRootPath = "/Game";
		static const FName EngineRootPath = "/Engine";

		if (InFolderPath == GameRootPath)
		{
			FolderDisplayNameOverride = LOCTEXT("GameFolderDisplayName", "Content");
		}
		else if (InFolderPath == EngineRootPath)
		{
			if (GetDefault<UContentBrowserSettings>()->bOrganizeFolders)
			{
				FolderDisplayNameOverride = LOCTEXT("EngineOrganizedFolderDisplayName", "Content");
			}
			else
			{
				FolderDisplayNameOverride = LOCTEXT("EngineFolderDisplayName", "Engine Content");
			}
		}
	}

	if (FolderDisplayNameOverride.IsEmpty())
	{
		if (IsTopLevelFolder(FStringView(FNameBuilder(InFolderPath))))
		{
			FStringView TopLevelFolderName(InFolderItemName);

			if (bIsClassesFolder)
			{
				static const FString ClassesPrefix = TEXT("Classes_");
				if (TopLevelFolderName.StartsWith(ClassesPrefix))
				{
					TopLevelFolderName.RightChopInline(ClassesPrefix.Len());
				}
			}
			
			// If we're using the Verse path of the plugin in the content browser, don't modify it's folder name
			const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TopLevelFolderName);
			if (Plugin)
			{
				UContentBrowserDataSubsystem* ContentBrowser = IContentBrowserDataModule::Get().GetSubsystem();
				if (ContentBrowser && ContentBrowser->UsePluginVersePath(Plugin.ToSharedRef()))
				{
					FolderDisplayNameOverride = FText::FromString(FPaths::GetPathLeaf(Plugin->GetVersePath()));
				}
			}

			if (FolderDisplayNameOverride.IsEmpty())
			{
				FString OverrideName;
				if (GetDefault<UContentBrowserSettings>()->bDisplayFriendlyNameForPluginFolders && Plugin && Plugin->GetFriendlyName().Len() > 0)
				{
					OverrideName = Plugin->GetFriendlyName();
				}
				if (OverrideName.IsEmpty())
				{
					OverrideName = TopLevelFolderName;
				}

				if (bIsClassesFolder)
				{
					FolderDisplayNameOverride = FText::Format(LOCTEXT("ClassFolderDisplayNameFmt", "{0} C++ Classes"), FText::AsCultureInvariant(OverrideName));
				}
				else
				{
					bool bDisplayContentFolderSuffix = GetDefault<UContentBrowserSettings>()->bDisplayContentFolderSuffix;
					if (bDisplayContentFolderSuffix && Plugin && Plugin->GetDescriptor().Modules.Num() == 0 && bIsCookedPath)
					{
						// Exclude the content suffix for plugins that only contain cooked content and have no C++ modules
						bDisplayContentFolderSuffix = false;
					}

					if (bDisplayContentFolderSuffix)
					{
						FolderDisplayNameOverride = FText::Format(LOCTEXT("ContentFolderDisplayNameFmt", "{0} Content"), FText::AsCultureInvariant(OverrideName));
					}
					else
					{
						FolderDisplayNameOverride = FText::AsCultureInvariant(OverrideName);
					}
				}
			}
		}
	}

	return FolderDisplayNameOverride;
}

bool ContentBrowserDataUtils::IsPrimaryAsset(const FAssetData& InAssetData)
{
	if (InAssetData.AssetClassPath == UVerseClass::VerseClassTopLevelAssetPath || InAssetData.AssetClassPath == UVerseStruct::VerseStructTopLevelAssetPath)
	{
		// We should not be showing generic compiled Verse classes, only `creative_device` subclasses and other types that 
		// can be drag-and-dropped into the level. We create aliases to display these elsewhere (ie: See FVerseClassAlias)
		return false;
	}
	
	// NOTE: (yiliang.siew) This tag would have been added during the cook for anything that is an internal-Verse-compiler-implementation detail.
	if (InAssetData.FindTag(Verse::InternalCodegenAssetRegistryTagName))
	{
		return false;
	}

	// Both GetOptionalOuterPathName and IsUAsset currently do not work on cooked assets
	//
	// GetOptionalOuterPathName is not serialized to the asset registry during cook
	// IsUAsset when called on compiled blueprint class compares Name_C vs Name and returns false
	if (InAssetData.HasAnyPackageFlags(PKG_Cooked | PKG_FilterEditorOnly))
	{
		// Check for the asset being a redirector first, as currently only class 
		// redirectors emit non-primary assets from the Asset Registry
		return !InAssetData.IsRedirector() || InAssetData.IsUAsset();
	}
	else
	{
		// External assets are not displayed in the Content Browser or other asset pickers
		bool bIsExternalAsset = !InAssetData.GetOptionalOuterPathName().IsNone();
		return !bIsExternalAsset && InAssetData.IsUAsset();
	}
}

bool ContentBrowserDataUtils::IsPrimaryAsset(UObject* InObject)
{
	if (const UVerseClass* VerseClass = Cast<UVerseClass>(InObject))
	{
		// We should not be showing generic compiled Verse classes, only `creative_device` subclasses and other types that 
		// can be drag-and-dropped into the level. We create aliases to display these elsewhere (ie: See FVerseClassAlias)
		return false;
	}
	if (Verse::IsInternalCodegenAsset(*InObject))
	{
		return false;
	}
	
	// External assets are not displayed in the Content Browser or other asset pickers
	const bool bIsExternalAsset = InObject->IsPackageExternal();

	return !bIsExternalAsset && FAssetData::IsUAsset(InObject);
}

#undef LOCTEXT_NAMESPACE
