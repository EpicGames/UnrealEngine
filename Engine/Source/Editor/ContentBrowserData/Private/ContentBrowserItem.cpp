// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserItem.h"
#include "ContentBrowserDataSource.h"
#include "IContentBrowserDataModule.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserItemPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBrowserItem)

#define LOCTEXT_NAMESPACE "ContentBrowserData"

FContentBrowserItem::FContentBrowserItem(FContentBrowserItemData&& InItem)
{
	checkf(InItem.IsValid(), TEXT("Items must be valid!"));
	ItemDataArray.Emplace(MoveTemp(InItem));
}

FContentBrowserItem::FContentBrowserItem(const FContentBrowserItemData& InItem)
{
	checkf(InItem.IsValid(), TEXT("Items must be valid!"));
	ItemDataArray.Emplace(InItem);
}

FContentBrowserItem::FContentBrowserItem(TArrayView<const FContentBrowserItemData> InItems)
	: ItemDataArray(InItems.GetData(), InItems.Num())
{
#if DO_CHECK
	FName PrimaryItemPath;
	EContentBrowserItemFlags PrimaryItemType = EContentBrowserItemFlags::None;
	for (const FContentBrowserItemData& ItemData : ItemDataArray)
	{
		checkf(ItemData.IsValid(), TEXT("Items must be valid!"));

		const FName ItemPath = ItemData.GetVirtualPath();
		const EContentBrowserItemFlags ItemType = ItemData.GetItemType();

		checkf(PrimaryItemPath.IsNone() || PrimaryItemPath == ItemPath, TEXT("All items must have the same path!"));
		checkf(PrimaryItemType == EContentBrowserItemFlags::None || PrimaryItemType == ItemType, TEXT("All items must be the same type!"));

		PrimaryItemPath = ItemPath;
		PrimaryItemType = ItemType;
	}
	checkf(ItemDataArray.Num() <= 1 || PrimaryItemType == EContentBrowserItemFlags::Type_Folder, TEXT("Only folders can contain multiple items!"));
#endif
}

FContentBrowserItem::FItemDataArrayView FContentBrowserItem::GetInternalItems() const
{
	return MakeArrayView(ItemDataArray);
}

const FContentBrowserItemData* FContentBrowserItem::GetPrimaryInternalItem() const
{
	return ItemDataArray.Num() > 0
		? &ItemDataArray[0]
		: nullptr;
}

bool FContentBrowserItem::IsValid() const
{
	return ItemDataArray.Num() > 0;
}

void FContentBrowserItem::Append(const FContentBrowserItem& InOther)
{
#if DO_CHECK
	FText ErrorText;
	if (!TryAppend(InOther, &ErrorText))
	{
		checkf(false, TEXT("Failed to append item '%s': %s"), *UContentBrowserItemLibrary::GetVirtualPath(InOther).ToString(), *ErrorText.ToString());
	}
#else
	TryAppend(InOther);
#endif
}

bool FContentBrowserItem::TryAppend(const FContentBrowserItem& InOther, FText* OutError)
{
	const FContentBrowserItemData* OtherPrimaryItemData = InOther.GetPrimaryInternalItem();
	if (!OtherPrimaryItemData)
	{
		// The other item is empty, nothing to do (but not a failure)
		return true;
	}

	// Try and append the primary item to us
	if (TryAppend(*OtherPrimaryItemData, OutError))
	{
		if (OtherPrimaryItemData->IsFolder())
		{
			// If this was a folder, also append any additional items
			for (const FContentBrowserItemData& OtherItemData : InOther.ItemDataArray)
			{
				// Skip the primary item as it was handled by the TryAppend call above
				if (OtherPrimaryItemData != &OtherItemData)
				{
					if (FContentBrowserItemData* FoundItem = ItemDataArray.FindByKey(OtherItemData))
					{
						// Assign the item to copy over any data that isn't considered part of its identity (like an updated payload)
						*FoundItem = OtherItemData;
					}
					else
					{
						ItemDataArray.Add(OtherItemData);
					}
				}
			}
			// TODO: Priority sort after merging?
		}
		return true;
	}

	return false;
}

void FContentBrowserItem::Append(const FContentBrowserItemData& InOther)
{
#if DO_CHECK
	FText ErrorText;
	if (!TryAppend(InOther, &ErrorText))
	{
		checkf(false, TEXT("Failed to append item '%s': %s"), *InOther.GetVirtualPath().ToString(), *ErrorText.ToString());
	}
#else
	TryAppend(InOther);
#endif
}

bool FContentBrowserItem::TryAppend(const FContentBrowserItemData& InOther, FText* OutError)
{
	const FContentBrowserItemData* PrimaryItemData = GetPrimaryInternalItem();
	if (!PrimaryItemData)
	{
		// We are empty, just copy the other item
		ItemDataArray.Add(InOther);
		return true;
	}

	if (PrimaryItemData->GetVirtualPath() != InOther.GetVirtualPath())
	{
		if (OutError)
		{
			*OutError = LOCTEXT("AppendError_DifferentPaths", "The items had different paths");
		}
		return false;
	}

	const EContentBrowserItemFlags PrimaryItemType = PrimaryItemData->GetItemType();
	if (PrimaryItemType != InOther.GetItemType())
	{
		if (OutError)
		{
			*OutError = LOCTEXT("AppendError_DifferentTypes", "The items had different types");
		}
		return false;
	}

	switch (PrimaryItemType)
	{
	case EContentBrowserItemFlags::Type_Folder:
		// Folders can have multiple items; just merge this item
		if (FContentBrowserItemData* FoundItem = ItemDataArray.FindByKey(InOther))
		{
			// Assign the item to copy over any data that isn't considered part of its identity (like an updated payload)
			*FoundItem = InOther;
		}
		else
		{
			ItemDataArray.Add(InOther);
		}
		// TODO: Priority sort after merging?
		break;

	case EContentBrowserItemFlags::Type_File:
		// Files can only have a single item; allow only if the item is already present
		if (ItemDataArray[0] == InOther)
		{
			// Assign the item to copy over any data that isn't considered part of its identity (like an updated payload)
			ItemDataArray[0] = InOther;
		}
		else
		{
			if (OutError)
			{
				*OutError = LOCTEXT("AppendError_FileSingleItem", "Files can only contain a single item");
			}
			return false;
		}
		break;

	default:
		checkf(false, TEXT("Unexpected EContentBrowserItemFlags::Type flag!"));
		break;
	}

	return true;
}

void FContentBrowserItem::Remove(const FContentBrowserItem& InOther)
{
#if DO_CHECK
	FText ErrorText;
	if (!TryRemove(InOther, &ErrorText))
	{
		checkf(false, TEXT("Failed to remove item '%s': %s"), *UContentBrowserItemLibrary::GetVirtualPath(InOther).ToString(), *ErrorText.ToString());
	}
#else
	TryRemove(InOther);
#endif
}

bool FContentBrowserItem::TryRemove(const FContentBrowserItem& InOther, FText* OutError)
{
	const FContentBrowserItemData* OtherPrimaryItemData = InOther.GetPrimaryInternalItem();
	if (!OtherPrimaryItemData)
	{
		// The other item is empty, nothing to do (but not a failure)
		return true;
	}

	// Try and remove the primary item from us
	if (TryRemove(*OtherPrimaryItemData, OutError))
	{
		if (OtherPrimaryItemData->IsFolder())
		{
			// If this was a folder, also remove any additional items
			ItemDataArray.RemoveAll([&InOther](const FContentBrowserItemData& InItemData)
			{
				return InOther.ItemDataArray.Contains(InItemData);
			});
		}
		return true;
	}

	return false;
}

void FContentBrowserItem::Remove(const FContentBrowserItemData& InOther)
{
#if DO_CHECK
	FText ErrorText;
	if (!TryRemove(InOther, &ErrorText))
	{
		checkf(false, TEXT("Failed to remove item '%s': %s"), *InOther.GetVirtualPath().ToString(), *ErrorText.ToString());
	}
#else
	TryRemove(InOther);
#endif
}

bool FContentBrowserItem::TryRemove(const FContentBrowserItemData& InOther, FText* OutError)
{
	const FContentBrowserItemData* PrimaryItemData = GetPrimaryInternalItem();
	if (!PrimaryItemData)
	{
		// We are empty, nothing to do (but not a failure)
		return true;
	}

	if (PrimaryItemData->GetVirtualPath() != InOther.GetVirtualPath())
	{
		if (OutError)
		{
			*OutError = LOCTEXT("RemoveError_DifferentPaths", "The items had different paths");
		}
		return false;
	}

	if (PrimaryItemData->GetItemType() != InOther.GetItemType())
	{
		if (OutError)
		{
			*OutError = LOCTEXT("RemoveError_DifferentTypes", "The items had different types");
		}
		return false;
	}

	// Remove the other item from our list
	ItemDataArray.Remove(InOther);
	return true;
}

FContentBrowserItemData FContentBrowserItem::TryRemove(const FContentBrowserMinimalItemData& InKey, FText* OutError)
{
	const FContentBrowserItemData* PrimaryItemData = GetPrimaryInternalItem();
	if (!PrimaryItemData)
	{
		// We are empty, nothing to do (but not a failure)
		return FContentBrowserItemData();
	}

	// Remove the other item from our list
	int32 Index = ItemDataArray.IndexOfByPredicate([Key=InKey](const FContentBrowserItemData& Data) { return Key == FContentBrowserMinimalItemData(Data); });
	if (Index != INDEX_NONE)
	{
		FContentBrowserItemData Data = MoveTemp(ItemDataArray[Index]);
		ItemDataArray.RemoveAt(Index);
		return MoveTemp(Data);
	}
	else
	{
		if (OutError)
		{
			*OutError = LOCTEXT("RemoveError_KeyMismatch", "No data matching given key");
		}
		return FContentBrowserItemData();
	}
}
bool FContentBrowserItem::IsFolder() const
{
	const FContentBrowserItemData* PrimaryItemData = GetPrimaryInternalItem();
	return PrimaryItemData && PrimaryItemData->IsFolder();
}

bool FContentBrowserItem::IsFile() const
{
	const FContentBrowserItemData* PrimaryItemData = GetPrimaryInternalItem();
	return PrimaryItemData && PrimaryItemData->IsFile();
}

bool FContentBrowserItem::IsInPlugin() const
{
	const FContentBrowserItemData* PrimaryItemData = GetPrimaryInternalItem();
	return PrimaryItemData && PrimaryItemData->IsPlugin();
}

bool FContentBrowserItem::IsSupported() const
{
	const FContentBrowserItemData* PrimaryItemData = GetPrimaryInternalItem();
	return PrimaryItemData && PrimaryItemData->IsSupported();
}

bool FContentBrowserItem::IsTemporary() const
{
	const FContentBrowserItemData* PrimaryItemData = GetPrimaryInternalItem();
	return PrimaryItemData && PrimaryItemData->IsTemporary();
}

bool FContentBrowserItem::IsDisplayOnlyFolder() const
{
	const FContentBrowserItemData* PrimaryItemData = GetPrimaryInternalItem();
	return PrimaryItemData && PrimaryItemData->IsDisplayOnlyFolder();
}

EContentBrowserItemFlags FContentBrowserItem::GetItemFlags() const
{
	const FContentBrowserItemData* PrimaryItemData = GetPrimaryInternalItem();
	return PrimaryItemData
		? PrimaryItemData->GetItemFlags()
		: EContentBrowserItemFlags::None;
}

EContentBrowserItemFlags FContentBrowserItem::GetItemType() const
{
	const FContentBrowserItemData* PrimaryItemData = GetPrimaryInternalItem();
	return PrimaryItemData
		? PrimaryItemData->GetItemType()
		: EContentBrowserItemFlags::None;
}

EContentBrowserItemFlags FContentBrowserItem::GetItemCategory() const
{
	const FContentBrowserItemData* PrimaryItemData = GetPrimaryInternalItem();
	return PrimaryItemData
		? PrimaryItemData->GetItemCategory()
		: EContentBrowserItemFlags::None;
}

EContentBrowserItemFlags FContentBrowserItem::GetItemTemporaryReason() const
{
	const FContentBrowserItemData* PrimaryItemData = GetPrimaryInternalItem();
	return PrimaryItemData
		? PrimaryItemData->GetItemTemporaryReason()
		: EContentBrowserItemFlags::None;
}

FName FContentBrowserItem::GetVirtualPath() const
{
	const FContentBrowserItemData* PrimaryItemData = GetPrimaryInternalItem();
	return PrimaryItemData
		? PrimaryItemData->GetVirtualPath()
		: FName();
}

FName FContentBrowserItem::GetInvariantPath() const
{
	const FContentBrowserItemData* PrimaryItemData = GetPrimaryInternalItem();
	return PrimaryItemData
		? PrimaryItemData->GetInvariantPath()
		: FName();
}

FName FContentBrowserItem::GetInternalPath() const
{
	const FContentBrowserItemData* PrimaryItemData = GetPrimaryInternalItem();
	return PrimaryItemData
		? PrimaryItemData->GetInternalPath()
		: FName();
}

FName FContentBrowserItem::GetItemName() const
{
	const FContentBrowserItemData* PrimaryItemData = GetPrimaryInternalItem();
	return PrimaryItemData
		? PrimaryItemData->GetItemName()
		: FName();
}

FText FContentBrowserItem::GetDisplayName() const
{
	const FContentBrowserItemData* PrimaryItemData = GetPrimaryInternalItem();
	return PrimaryItemData
		? PrimaryItemData->GetDisplayName()
		: FText();
}

bool FContentBrowserItem::GetItemAssetAccessSpecifier(EAssetAccessSpecifier& OutAssetAccessSpecifier) const
{
	const FContentBrowserItemData* PrimaryItemData = GetPrimaryInternalItem();
	return PrimaryItemData
		? PrimaryItemData->GetItemAssetAccessSpecifier(OutAssetAccessSpecifier)
		: false;
}

bool FContentBrowserItem::CanModifyItemAssetAccessSpecifier() const
{
	const FContentBrowserItemData* PrimaryItemData = GetPrimaryInternalItem();
	return PrimaryItemData
		? PrimaryItemData->CanModifyItemAssetAccessSpecifier()
		: false;
}

struct FContentBrowserItemHelper
{
public:
	template <typename ObjectType, typename FuncType, typename... ArgTypes>
	static bool CallDataSourceImpl(const FContentBrowserItem& Item, FuncType Func, ArgTypes&&... Args)
	{
		bool bResult = false;
		static const FName RootPath = "/";

		FContentBrowserItem::FItemDataArrayView ItemDataArray = Item.GetInternalItems();
		for (const FContentBrowserItemData& ItemData : ItemDataArray)
		{
			if (UContentBrowserDataSource* ItemDataSource = ItemData.GetOwnerDataSource())
			{
				// Test the mount point again, as dummy items may have been emitted to represent the mount point itself
				if (/*ItemDataSource->IsVirtualPathUnderMountRoot(ItemData.GetVirtualPath()) && */(RootPath != ItemData.GetVirtualPath()))
				{
					if (ObjectType* CastItemDataSource = Cast<ObjectType>(ItemDataSource))
					{
						bResult |= Invoke(Func, CastItemDataSource, ItemData, Forward<ArgTypes>(Args)...);
					}
				}
			}
		}

		return bResult;
	}
};

FContentBrowserItemDataAttributeValue FContentBrowserItem::GetItemAttribute(const FName InAttributeKey, const bool InIncludeMetaData) const
{
	FContentBrowserItemDataAttributeValue AttributeValue;
	FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, GetItemAttribute), InIncludeMetaData, InAttributeKey, AttributeValue);
	return AttributeValue;
}

FContentBrowserItemDataAttributeValues FContentBrowserItem::GetItemAttributes(const bool InIncludeMetaData) const
{
	FContentBrowserItemDataAttributeValues AttributeValues;
	FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, GetItemAttributes), InIncludeMetaData, AttributeValues);
	return AttributeValues;
}

bool FContentBrowserItem::GetItemPhysicalPath(FString& OutDiskPath) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, GetItemPhysicalPath), OutDiskPath);
}

bool FContentBrowserItem::IsDirty() const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, IsItemDirty));
}

bool FContentBrowserItem::CanEdit(FText* OutErrorMsg) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, CanEditItem), OutErrorMsg);
}

bool FContentBrowserItem::Edit() const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, EditItem));
}

bool FContentBrowserItem::CanPreview(FText* OutErrorMsg) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, CanPreviewItem), OutErrorMsg);
}

bool FContentBrowserItem::CanView(FText* OutErrorMsg) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, CanViewItem), OutErrorMsg);
}

bool FContentBrowserItem::Preview() const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, PreviewItem));
}

bool FContentBrowserItem::CanDuplicate(FText* OutErrorMsg) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, CanDuplicateItem), OutErrorMsg);
}

FContentBrowserItemDataTemporaryContext FContentBrowserItem::Duplicate() const
{
	FContentBrowserItemDataTemporaryContext NewItem;
	FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, DuplicateItem), NewItem);
	return NewItem;
}

bool FContentBrowserItem::CanSave(const EContentBrowserItemSaveFlags InSaveFlags, FText* OutErrorMsg) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, CanSaveItem), InSaveFlags, OutErrorMsg);
}

bool FContentBrowserItem::Save(const EContentBrowserItemSaveFlags InSaveFlags) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, SaveItem), InSaveFlags);
}

bool FContentBrowserItem::CanDelete(FText* OutErrorMsg) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, CanDeleteItem), OutErrorMsg);
}

bool FContentBrowserItem::Delete() const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, DeleteItem));
}

bool FContentBrowserItem::CanRename(const FString* InNewName, FText* OutErrorMsg) const
{
	TSharedPtr<IContentBrowserHideFolderIfEmptyFilter> HideFolderIfEmptyFilter;
	if (InNewName != nullptr)
	{
		// Only create the filter if checking against a specific name, otherwise we shouldn't need it.
		const UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
		HideFolderIfEmptyFilter = ContentBrowserData->CreateHideFolderIfEmptyFilter();
	}

	return CanRename(InNewName, HideFolderIfEmptyFilter.Get(), OutErrorMsg);
}

bool FContentBrowserItem::CanRename(const FString* InNewName, const IContentBrowserHideFolderIfEmptyFilter* HideFolderIfEmptyFilter, FText* OutErrorMsg) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, CanRenameItem), InNewName, HideFolderIfEmptyFilter, OutErrorMsg);
}

bool FContentBrowserItem::Rename(const FString& InNewName, FContentBrowserItem* OutNewItem) const
{
	static const FName RootPath = "/";

	FContentBrowserItem NewItem;

	for (const FContentBrowserItemData& ItemData : ItemDataArray)
	{
		if (UContentBrowserDataSource* ItemDataSource = ItemData.GetOwnerDataSource())
		{
			// Test the mount point again, as dummy items may have been emitted to represent the mount point itself
			if (ItemDataSource->IsVirtualPathUnderMountRoot(ItemData.GetVirtualPath()) && (ItemData.GetVirtualPath() != RootPath))
			{
				FContentBrowserItemData NewItemData;
				if (ItemDataSource->RenameItem(ItemData, InNewName, NewItemData))
				{
					NewItem.Append(MoveTemp(NewItemData));
				}
			}
		}
	}

	if (NewItem.IsValid())
	{
		if (OutNewItem)
		{
			*OutNewItem = NewItem;
		}
		return true;
	}

	return false;
}

bool FContentBrowserItem::CanCopy(const FName InDestPath, FText* OutErrorMsg) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, CanCopyItem), InDestPath, OutErrorMsg);
}

bool FContentBrowserItem::Copy(const FName InDestPath) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, CopyItem), InDestPath);
}

bool FContentBrowserItem::CanMove(const FName InDestPath, FText* OutErrorMsg) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, CanMoveItem), InDestPath, OutErrorMsg);
}

bool FContentBrowserItem::Move(const FName InDestPath) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, MoveItem), InDestPath);
}

bool FContentBrowserItem::AppendItemReference(FString& InOutStr) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, AppendItemReference), InOutStr);
}

bool FContentBrowserItem::AppendItemObjectPath(FString& InOutStr) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, AppendItemObjectPath), InOutStr);
}

bool FContentBrowserItem::AppendItemPackageName(FString& InOutStr) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, AppendItemPackageName), InOutStr);
}

bool FContentBrowserItem::UpdateThumbnail(FAssetThumbnail& InThumbnail) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, UpdateThumbnail), InThumbnail);
}

bool FContentBrowserItem::TryGetCollectionId(FSoftObjectPath& OutCollectionId) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, TryGetCollectionId), OutCollectionId);
}

bool FContentBrowserItem::Legacy_TryGetPackagePath(FName& OutPackagePath) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, Legacy_TryGetPackagePath), OutPackagePath);
}

bool FContentBrowserItem::Legacy_TryGetAssetData(FAssetData& OutAssetData) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, Legacy_TryGetAssetData), OutAssetData);
}

bool FContentBrowserItem::GetItemAliasedPath(FSoftObjectPath& OutObjectPath) const
{
	return FContentBrowserItemHelper::CallDataSourceImpl<UContentBrowserDataSource>(*this, UE_PROJECTION_MEMBER(UContentBrowserDataSource, GetItemAliasedPath), OutObjectPath);
}

FContentBrowserItemPath FContentBrowserItem::GetItemPath() const
{
	const FName VirtualPath = GetVirtualPath();
	const FName InternalPath = GetInternalPath();

	if (VirtualPath.IsNone() && InternalPath.IsNone())
	{
		return FContentBrowserItemPath();
	}

	if (InternalPath.IsNone())
	{
		return FContentBrowserItemPath(VirtualPath, EContentBrowserPathType::Virtual);
	}
	
	if (VirtualPath.IsNone())
	{
		return FContentBrowserItemPath(InternalPath, EContentBrowserPathType::Internal);
	}
	
	return FContentBrowserItemPath(VirtualPath, InternalPath);
}


bool UContentBrowserItemLibrary::IsFolder(const FContentBrowserItem& Item)
{
	return Item.IsFolder();
}

bool UContentBrowserItemLibrary::IsFile(const FContentBrowserItem& Item)
{
	return Item.IsFile();
}

FName UContentBrowserItemLibrary::GetVirtualPath(const FContentBrowserItem& Item)
{
	return Item.GetVirtualPath();
}

FText UContentBrowserItemLibrary::GetDisplayName(const FContentBrowserItem& Item)
{
	return Item.GetDisplayName();
}


bool FContentBrowserItemTemporaryContext::IsValid() const
{
	return ItemDataContextArray.Num() > 0;
}

void FContentBrowserItemTemporaryContext::AppendContext(FContentBrowserItemDataTemporaryContext&& InContext)
{
	checkf(InContext.IsValid(), TEXT("FContentBrowserItemTemporaryContext should only be used with valid contexts!"));
	Item.Append(InContext.GetItemData());
	ItemDataContextArray.Emplace(MoveTemp(InContext));
}

const FContentBrowserItem& FContentBrowserItemTemporaryContext::GetItem() const
{
	return Item;
}

bool FContentBrowserItemTemporaryContext::ValidateItem(const FString& InProposedName, FText* OutErrorMsg) const
{
	for (const FContentBrowserItemDataTemporaryContext& Context : ItemDataContextArray)
	{
		if (!Context.ValidateItem(InProposedName, OutErrorMsg))
		{
			return false;
		}
	}

	return IsValid();
}

FContentBrowserItem FContentBrowserItemTemporaryContext::FinalizeItem(const FString& InProposedName, FText* OutErrorMsg) const
{
	FContentBrowserItem NewItem;

	for (const FContentBrowserItemDataTemporaryContext& Context : ItemDataContextArray)
	{
		FContentBrowserItemData NewItemData = Context.FinalizeItem(InProposedName, OutErrorMsg);
		if (NewItemData.IsValid())
		{
			NewItem.Append(MoveTemp(NewItemData));
		}
	}

	return NewItem;
}


FContentBrowserItemKey::FContentBrowserItemKey(const FContentBrowserItem& InItem)
{
	if (const FContentBrowserItemData* PrimaryItemData = InItem.GetPrimaryInternalItem())
	{
		Initialize(*PrimaryItemData);
	}
}

FContentBrowserItemKey::FContentBrowserItemKey(const FContentBrowserItemData& InItemData)
{
	Initialize(InItemData);
}

FContentBrowserItemKey::FContentBrowserItemKey(EContentBrowserItemFlags InItemType, FName InPath, const UContentBrowserDataSource* InDataSource)
	: FContentBrowserItemDataKey(InItemType, InPath)
{
	if (ItemType == EContentBrowserItemFlags::Type_File)
	{
		DataSource = InDataSource;
	}
}

void FContentBrowserItemKey::Initialize(const FContentBrowserItemData& InItemData)
{
	ItemType = InItemData.GetItemType();
	VirtualPath = InItemData.GetVirtualPath();
	if (ItemType == EContentBrowserItemFlags::Type_File)
	{
		DataSource = InItemData.GetOwnerDataSource();
	}
}


FContentBrowserItemUpdate::FContentBrowserItemUpdate(const FContentBrowserItemDataUpdate& InItemDataUpdate)
	: UpdateType(InItemDataUpdate.GetUpdateType())
	, PreviousVirtualPath(InItemDataUpdate.GetPreviousVirtualPath())
{
	if (InItemDataUpdate.GetItemData().IsValid())
	{
		ItemData = FContentBrowserItem(InItemDataUpdate.GetItemData());
	}
}

EContentBrowserItemUpdateType FContentBrowserItemUpdate::GetUpdateType() const
{
	return UpdateType;
}

const FContentBrowserItem& FContentBrowserItemUpdate::GetItemData() const
{
	return ItemData;
}

FName FContentBrowserItemUpdate::GetPreviousVirtualPath() const
{
	return PreviousVirtualPath;
}

#undef LOCTEXT_NAMESPACE
