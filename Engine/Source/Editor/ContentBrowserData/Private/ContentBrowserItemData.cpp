// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserItemData.h"

#include "Containers/StringView.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserDataSubsystem.h"
#include "Misc/AssertionMacros.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/UnrealNames.h"

FContentBrowserItemData::FContentBrowserItemData(
	UContentBrowserDataSource* InOwnerDataSource,
	EContentBrowserItemFlags InItemFlags,
	FName InVirtualPath,
	FName InItemName,
	FText InDisplayNameOverride,
	TSharedPtr<const IContentBrowserItemDataPayload> InPayload,
	FName InInternalPath)
	: OwnerDataSource(InOwnerDataSource)
	, ItemFlags(InItemFlags)
	, VirtualPath(InVirtualPath)
	, ItemName(InItemName)
	, InternalPath(InInternalPath)
	, CachedDisplayName(MoveTemp(InDisplayNameOverride))
	, Payload(MoveTemp(InPayload))
{
	checkf(IsFolder() != IsFile(), TEXT("Items must be either a folder or a file!"));
	if (CachedDisplayName.IsEmpty())
	{
		CachedDisplayName = FText::AsCultureInvariant(ItemName.ToString());
	}
}

bool FContentBrowserItemData::operator==(const FContentBrowserItemData& InOther) const
{
	// Item data is considered equal if it has same type and path, and is from the same data source
	return OwnerDataSource == InOther.OwnerDataSource
		&& GetItemType() == InOther.GetItemType()
		&& VirtualPath == InOther.VirtualPath;
}

bool FContentBrowserItemData::operator!=(const FContentBrowserItemData& InOther) const
{
	return !(*this == InOther);
}

bool FContentBrowserItemData::IsValid() const
{
	return GetItemType() != EContentBrowserItemFlags::None;
}

bool FContentBrowserItemData::IsFolder() const
{
	return EnumHasAnyFlags(ItemFlags, EContentBrowserItemFlags::Type_Folder);
}

bool FContentBrowserItemData::IsFile() const
{
	return EnumHasAnyFlags(ItemFlags, EContentBrowserItemFlags::Type_File);
}

bool FContentBrowserItemData::IsPlugin() const
{
	return EnumHasAnyFlags(ItemFlags, EContentBrowserItemFlags::Category_Plugin);
}

bool FContentBrowserItemData::IsSupported() const
{
	return !EnumHasAnyFlags(ItemFlags, EContentBrowserItemFlags::Misc_Unsupported);
}

bool FContentBrowserItemData::IsTemporary() const
{
	return EnumHasAnyFlags(ItemFlags, EContentBrowserItemFlags::Temporary_MASK);
}

bool FContentBrowserItemData::IsDisplayOnlyFolder() const
{
	return GetItemCategory() == EContentBrowserItemFlags::None;
}

UContentBrowserDataSource* FContentBrowserItemData::GetOwnerDataSource() const
{
	return OwnerDataSource.Get();
}

EContentBrowserItemFlags FContentBrowserItemData::GetItemFlags() const
{
	return ItemFlags;
}

EContentBrowserItemFlags FContentBrowserItemData::GetItemType() const
{
	return ItemFlags & EContentBrowserItemFlags::Type_MASK;
}

EContentBrowserItemFlags FContentBrowserItemData::GetItemCategory() const
{
	return ItemFlags & EContentBrowserItemFlags::Category_MASK;
}

EContentBrowserItemFlags FContentBrowserItemData::GetItemTemporaryReason() const
{
	return ItemFlags & EContentBrowserItemFlags::Temporary_MASK;
}

FName FContentBrowserItemData::GetVirtualPath() const
{
	return VirtualPath;
}

FName FContentBrowserItemData::GetInvariantPath() const
{
	FName Internal = GetInternalPath();
	if (!Internal.IsNone())
	{
		return Internal;
	}
	else if (!VirtualPath.IsNone())
	{
		if (UContentBrowserDataSource* DataSource = OwnerDataSource.Get())
		{
			FName ConvertedPath;
			DataSource->TryConvertVirtualPath(VirtualPath, ConvertedPath);
			return ConvertedPath;
		}
	}

	return VirtualPath;
}

FName FContentBrowserItemData::GetInternalPath() const
{
	return InternalPath;
}

FName FContentBrowserItemData::GetItemName() const
{
	return ItemName;
}

FText FContentBrowserItemData::GetDisplayName() const
{
	return CachedDisplayName;
}

bool FContentBrowserItemData::GetItemAssetAccessSpecifier(EAssetAccessSpecifier& OutAssetAccessSpecifier) const
{
	if (UContentBrowserDataSource* ItemDataSource = GetOwnerDataSource())
	{
		return ItemDataSource->GetItemAssetAccessSpecifier(*this, OutAssetAccessSpecifier);
	}

	return false;
}

bool FContentBrowserItemData::CanModifyItemAssetAccessSpecifier() const
{
	if (UContentBrowserDataSource* ItemDataSource = GetOwnerDataSource())
	{
		return ItemDataSource->CanModifyItemAssetAccessSpecifier(*this);
	}

	return false;
}

TSharedPtr<const IContentBrowserItemDataPayload> FContentBrowserItemData::GetPayload() const
{
	return Payload;
}


FContentBrowserItemDataAttributeValue::FContentBrowserItemDataAttributeValue(const TCHAR* InStr)
	: ValueType(EContentBrowserItemDataAttributeValueType::String)
	, StrValue(InStr)
{
}

FContentBrowserItemDataAttributeValue::FContentBrowserItemDataAttributeValue(const FString& InStr)
	: ValueType(EContentBrowserItemDataAttributeValueType::String)
	, StrValue(InStr)
{
}

FContentBrowserItemDataAttributeValue::FContentBrowserItemDataAttributeValue(FString&& InStr)
	: ValueType(EContentBrowserItemDataAttributeValueType::String)
	, StrValue(MoveTemp(InStr))
{
}

FContentBrowserItemDataAttributeValue::FContentBrowserItemDataAttributeValue(const FName InName)
	: ValueType(EContentBrowserItemDataAttributeValueType::Name)
	, NameValue(InName)
{
}

FContentBrowserItemDataAttributeValue::FContentBrowserItemDataAttributeValue(FText InText)
	: ValueType(EContentBrowserItemDataAttributeValueType::Text)
	, TextValue(MoveTemp(InText))
{
}

bool FContentBrowserItemDataAttributeValue::IsValid() const
{
	return ValueType != EContentBrowserItemDataAttributeValueType::None;
}

void FContentBrowserItemDataAttributeValue::Reset()
{
	switch (ValueType)
	{
	case EContentBrowserItemDataAttributeValueType::String:
		StrValue.Reset();
		break;

	case EContentBrowserItemDataAttributeValueType::Name:
		NameValue = FName();
		break;

	case EContentBrowserItemDataAttributeValueType::Text:
		TextValue = FText();
		break;
	}

	ValueType = EContentBrowserItemDataAttributeValueType::None;
	MetaData.Reset();
}

EContentBrowserItemDataAttributeValueType FContentBrowserItemDataAttributeValue::GetValueType() const
{
	return ValueType;
}

void FContentBrowserItemDataAttributeValue::SetValue(const TCHAR* InStr)
{
	Reset();

	ValueType = EContentBrowserItemDataAttributeValueType::String;
	StrValue = InStr;
}

void FContentBrowserItemDataAttributeValue::SetValue(const FString& InStr)
{
	Reset();

	ValueType = EContentBrowserItemDataAttributeValueType::String;
	StrValue = InStr;
}

void FContentBrowserItemDataAttributeValue::SetValue(FString&& InStr)
{
	Reset();

	ValueType = EContentBrowserItemDataAttributeValueType::String;
	StrValue = MoveTemp(InStr);
}

void FContentBrowserItemDataAttributeValue::SetValue(const FName InName)
{
	Reset();

	ValueType = EContentBrowserItemDataAttributeValueType::Name;
	NameValue = InName;
}

void FContentBrowserItemDataAttributeValue::SetValue(FText InText)
{
	Reset();

	ValueType = EContentBrowserItemDataAttributeValueType::Text;
	TextValue = MoveTemp(InText);
}

FStringView FContentBrowserItemDataAttributeValue::GetValueStringView(FStringBuilderBase& ScratchBuffer) const
{
	switch (ValueType)
	{
	case EContentBrowserItemDataAttributeValueType::String:
		return FStringView(StrValue);

	case EContentBrowserItemDataAttributeValueType::Name:
		NameValue.ToString(ScratchBuffer);
		return FStringView(ScratchBuffer);

	case EContentBrowserItemDataAttributeValueType::Text:
		return FStringView(TextValue.ToString());
	}

	return FStringView();
}

const FString& FContentBrowserItemDataAttributeValue::GetValueString() const
{
	check(ValueType == EContentBrowserItemDataAttributeValueType::String);
	return StrValue;
}

FName FContentBrowserItemDataAttributeValue::GetValueName() const
{
	check(ValueType == EContentBrowserItemDataAttributeValueType::Name);
	return NameValue;
}

FText FContentBrowserItemDataAttributeValue::GetValueText() const
{
	check(ValueType == EContentBrowserItemDataAttributeValueType::Text);
	return TextValue;
}

const FContentBrowserItemDataAttributeMetaData& FContentBrowserItemDataAttributeValue::GetMetaData() const
{
	static const FContentBrowserItemDataAttributeMetaData EmptyMetaData = FContentBrowserItemDataAttributeMetaData();
	return MetaData ? *MetaData : EmptyMetaData;
}

void FContentBrowserItemDataAttributeValue::SetMetaData(const FContentBrowserItemDataAttributeMetaData& InMetaData)
{
	MetaData = MakeShared<FContentBrowserItemDataAttributeMetaData>(InMetaData);
}

void FContentBrowserItemDataAttributeValue::SetMetaData(FContentBrowserItemDataAttributeMetaData&& InMetaData)
{
	MetaData = MakeShared<FContentBrowserItemDataAttributeMetaData>(MoveTemp(InMetaData));
}


FContentBrowserItemDataTemporaryContext::FContentBrowserItemDataTemporaryContext(FContentBrowserItemData&& InItemData, FOnValidateItem InOnValidateItem, FOnFinalizeItem InOnFinalizeItem)
	: ItemData(MoveTemp(InItemData))
	, OnValidateItem(InOnValidateItem)
	, OnFinalizeItem(InOnFinalizeItem)
{
	checkf(ItemData.IsTemporary(), TEXT("FContentBrowserItemDataTemporaryContext can only be used with temporary items!"));
	checkf(OnFinalizeItem.IsBound(), TEXT("FContentBrowserItemDataTemporaryContext must have a valid OnFinalizeItem!"));
}

bool FContentBrowserItemDataTemporaryContext::IsValid() const
{
	return ItemData.IsValid();
}

const FContentBrowserItemData& FContentBrowserItemDataTemporaryContext::GetItemData() const
{
	return ItemData;
}

bool FContentBrowserItemDataTemporaryContext::ValidateItem(const FString& InProposedName, FText* OutErrorMsg) const
{
	return !OnValidateItem.IsBound() || OnValidateItem.Execute(ItemData, InProposedName, OutErrorMsg);
}

FContentBrowserItemData FContentBrowserItemDataTemporaryContext::FinalizeItem(const FString& InProposedName, FText* OutErrorMsg) const
{
	return OnFinalizeItem.Execute(ItemData, InProposedName, OutErrorMsg);
}

FContentBrowserMinimalItemData::FContentBrowserMinimalItemData(const FContentBrowserItemData& InItemData)
	: FContentBrowserMinimalItemData(InItemData.GetItemType(), InItemData.GetVirtualPath(), InItemData.GetOwnerDataSource())
{

}

FContentBrowserMinimalItemData::FContentBrowserMinimalItemData(EContentBrowserItemFlags InItemType, FName InVirtualPath, const UContentBrowserDataSource* InSource)
	: ItemType(InItemType)
	, VirtualPath(InVirtualPath)
	, DataSource(InSource)
{

}

FString FContentBrowserMinimalItemData::ToString() const
{
	return FString::Printf(TEXT("%s:%s:%s"), 
		*VirtualPath.ToString(), 
		EnumHasAllFlags(ItemType, EContentBrowserItemFlags::Type_File) ? TEXT("File") : TEXT("Folder"),
		DataSource ? *DataSource->GetName() : TEXT("null")
	);
}

FContentBrowserItemDataKey::FContentBrowserItemDataKey(const FContentBrowserItemData& InItemData)
	: ItemType(InItemData.GetItemType())
	, VirtualPath(InItemData.GetVirtualPath())
{
}

FContentBrowserItemDataKey::FContentBrowserItemDataKey(EContentBrowserItemFlags InItemType, FName InVirtualPath)
	: ItemType(InItemType & EContentBrowserItemFlags::Type_MASK)
	, VirtualPath(InVirtualPath)
{
	checkf(EnumHasAnyFlags(ItemType, EContentBrowserItemFlags::Type_Folder) != EnumHasAnyFlags(ItemType, EContentBrowserItemFlags::Type_File), TEXT("Items must be either a folder or a file!"));
}


FContentBrowserItemDataUpdate FContentBrowserItemDataUpdate::MakeItemAddedUpdate(FContentBrowserItemData InItemData)
{
	FContentBrowserItemDataUpdate ItemUpdate;
	ItemUpdate.UpdateType = EContentBrowserItemUpdateType::Added;
	ItemUpdate.ItemData = MoveTemp(InItemData);
	return ItemUpdate;
}

FContentBrowserItemDataUpdate FContentBrowserItemDataUpdate::MakeItemModifiedUpdate(FContentBrowserItemData InItemData)
{
	FContentBrowserItemDataUpdate ItemUpdate;
	ItemUpdate.UpdateType = EContentBrowserItemUpdateType::Modified;
	ItemUpdate.ItemData = MoveTemp(InItemData);
	return ItemUpdate;
}

FContentBrowserItemDataUpdate FContentBrowserItemDataUpdate::MakeItemMovedUpdate(FContentBrowserItemData InItemData, FName InPreviousVirtualPath)
{
	FContentBrowserItemDataUpdate ItemUpdate;
	ItemUpdate.UpdateType = EContentBrowserItemUpdateType::Moved;
	ItemUpdate.ItemData = MoveTemp(InItemData);
	ItemUpdate.PreviousVirtualPath = InPreviousVirtualPath;
	return ItemUpdate;
}

FContentBrowserItemDataUpdate FContentBrowserItemDataUpdate::MakeItemRemovedUpdate(FContentBrowserItemData InItemData)
{
	FContentBrowserItemDataUpdate ItemUpdate;
	ItemUpdate.UpdateType = EContentBrowserItemUpdateType::Removed;
	ItemUpdate.ItemData = MoveTemp(InItemData);
	return ItemUpdate;
}

FContentBrowserItemDataUpdate FContentBrowserItemDataUpdate::MakeItemUnmountedUpdate(FContentBrowserItemData InItemData)
{
	FContentBrowserItemDataUpdate ItemUpdate;
	ItemUpdate.UpdateType = EContentBrowserItemUpdateType::Unmounted;
	ItemUpdate.ItemData = MoveTemp(InItemData);
	return ItemUpdate;
}

EContentBrowserItemUpdateType FContentBrowserItemDataUpdate::GetUpdateType() const
{
	return UpdateType;
}

const FContentBrowserItemData& FContentBrowserItemDataUpdate::GetItemData() const
{
	return ItemData;
}

FName FContentBrowserItemDataUpdate::GetPreviousVirtualPath() const
{
	return PreviousVirtualPath;
}
