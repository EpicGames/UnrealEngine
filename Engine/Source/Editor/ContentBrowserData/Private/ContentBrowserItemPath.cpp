// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserItemPath.h"

#include "Containers/StringView.h"
#include "IContentBrowserDataModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBrowserItemPath)

FContentBrowserItemPath::FContentBrowserItemPath()
{
}

FContentBrowserItemPath::FContentBrowserItemPath(const FStringView InPath, const EContentBrowserPathType InPathType)
{
	SetPathFromString(InPath, InPathType);
}

FContentBrowserItemPath::FContentBrowserItemPath(const TCHAR* InPath, const EContentBrowserPathType InPathType)
{
	SetPathFromString(FStringView(InPath), InPathType);
}

FContentBrowserItemPath::FContentBrowserItemPath(const FName InPath, const EContentBrowserPathType InPathType)
{
	SetPathFromName(InPath, InPathType);
}

FName FContentBrowserItemPath::GetVirtualPathName() const
{
	return VirtualPath;
}

FName FContentBrowserItemPath::GetInternalPathName() const
{
	check(!InternalPath.IsNone());
	return InternalPath;
}

FString FContentBrowserItemPath::GetVirtualPathString() const
{
	return VirtualPath.ToString();
}

FString FContentBrowserItemPath::GetInternalPathString() const
{
	check(!InternalPath.IsNone());
	return InternalPath.ToString();
}

bool FContentBrowserItemPath::HasInternalPath() const
{
	return !InternalPath.IsNone();
}

void FContentBrowserItemPath::SetPathFromString(const FStringView InPath, const EContentBrowserPathType InPathType)
{
	SetPathFromName(FName(InPath), InPathType);
}

void FContentBrowserItemPath::SetPathFromName(const FName InPath, const EContentBrowserPathType InPathType)
{
	if (InPathType == EContentBrowserPathType::Virtual)
	{
		VirtualPath = InPath;
		const EContentBrowserPathType AssetPathType = IContentBrowserDataModule::Get().GetSubsystem()->TryConvertVirtualPath(InPath, InternalPath);
		if (AssetPathType != EContentBrowserPathType::Internal)
		{
			InternalPath = NAME_None;
		}
	}
	else if (InPathType == EContentBrowserPathType::Internal)
	{
		InternalPath = InPath;
		IContentBrowserDataModule::Get().GetSubsystem()->ConvertInternalPathToVirtual(InPath, VirtualPath);
	}
	else
	{
		InternalPath = NAME_None;
		VirtualPath = NAME_None;
	}
}
