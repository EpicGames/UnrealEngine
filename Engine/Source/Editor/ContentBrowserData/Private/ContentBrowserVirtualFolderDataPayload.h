// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserItemData.h"

class FContentBrowserVirtualFolderItemDataPayload : public IContentBrowserItemDataPayload
{
public:
	explicit FContentBrowserVirtualFolderItemDataPayload (bool InIsCustomVirtual)
        : bIsCustomVirtualFolder(InIsCustomVirtual)
	{
	}

private:
    bool bIsCustomVirtualFolder;
};
