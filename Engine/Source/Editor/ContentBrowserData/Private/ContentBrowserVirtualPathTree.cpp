// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserVirtualPathTree.h"

#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDataSubsystem.h"
#include "IContentBrowserDataModule.h"
#include "Misc/AssertionMacros.h"
#include "Misc/PathViews.h"
#include "Misc/StringBuilder.h"
#include "Settings/ContentBrowserSettings.h"
#include "Templates/Function.h"
#include "Templates/Tuple.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"

void FContentBrowserVirtualPathTree::Reset()
{
	VirtualToInternalMounts.Reset();
	ParentPathToChildPaths.Reset();
	ChildPathToParentPath.Reset();
}

FStringView FContentBrowserVirtualPathTree::GetMountPointFromPath(const FStringView InPath, bool& bOutHadClassesPrefix)
{
	return FPathViews::GetMountPointNameFromPath(InPath, &bOutHadClassesPrefix);
}

bool FContentBrowserVirtualPathTree::CachePath(FName Path, FName InternalPath, TFunctionRef<void(FName)> OnPathAdded)
{
	if (Path.IsNone())
	{
		return false;
	}

	if (ParentPathToChildPaths.Contains(Path))
	{
		// Already cached - nothing more to do
		return false;
	}

	FString PathStr = Path.ToString();
	check(PathStr.Len() >= 2);	// Must be at least "/A"
	check(PathStr[0] == '/');	// Must start with a "/"

	// Paths are cached without their trailing slash, so if the given path has a trailing slash, test it again now as it may already be cached
	if (PathStr[PathStr.Len() - 1] == '/')
	{
		PathStr.RemoveAt(PathStr.Len() - 1, EAllowShrinking::No);
		Path = *PathStr;

		if (ParentPathToChildPaths.Contains(Path))
		{
			// Already cached - nothing more to do
			return false;
		}
	}

	FName LastPath;

	// Ensure an entry for the root of the path
	{
		static const FName PathRoot = "/";

		if (!ParentPathToChildPaths.Contains(PathRoot))
		{
			ParentPathToChildPaths.Add(PathRoot);
		}

		LastPath = PathRoot;
	}

	// Walk each part of the path, adding known path entries if required
	// This manipulates PathStr in-place to avoid making any string copies
	TCHAR* PathCharPtr = &PathStr[1]; // Skip over the first / when scanning
	for (;;)
	{
		const TCHAR PathChar = *PathCharPtr;
		if (PathChar == '/' || PathChar == 0)
		{
			// We've found a path separator (or the end of the string), so process this part of the path
			(*PathCharPtr) = 0;			// Null terminate this part of the string so we can create an FName from it
			const FName CurrentPath = *PathStr;
			(*PathCharPtr) = PathChar;	// Restore the original character now

			check(!CurrentPath.IsNone());	// Path parts cannot be empty
			check(*(PathCharPtr-1) != '/'); // The previous character cannot be a /, as that would suggest a malformed path such as "/Game//MyAsset"

			bool bAddedPath = false;
			if (!ParentPathToChildPaths.Contains(CurrentPath))
			{
				ParentPathToChildPaths.Add(CurrentPath);
				bAddedPath = true;
			}

			if (!LastPath.IsNone())
			{
				// Add us as a known child of our parent path
				TSet<FName>& ChildPaths = ParentPathToChildPaths.FindChecked(LastPath);
				ChildPaths.Add(CurrentPath);

				// Make sure we know how to find our parent again later on
				ChildPathToParentPath.Add(CurrentPath, LastPath);
			}

			if (bAddedPath)
			{
				OnPathAdded(CurrentPath);
			}

			LastPath = CurrentPath;
		}

		if (PathChar == 0)
		{
			// End of the string
			break;
		}

		++PathCharPtr;
	}

	VirtualToInternalMounts.Add(Path, InternalPath);

	return true;
}

bool FContentBrowserVirtualPathTree::RemovePath(FName Path, TFunctionRef<void(FName)> OnPathRemoved)
{
	if (Path.IsNone())
	{
		return false;
	}

	if (!ParentPathToChildPaths.Contains(Path))
	{
		// Paths are cached without their trailing slash, so if the given path has a trailing slash, test it again now as it may already be cached
		// We do this after the initial map test as: a) Most paths are well formed, b) This avoids string allocations until we know we need them
		FString PathStr = Path.ToString();
		if (PathStr[PathStr.Len() - 1] == '/')
		{
			PathStr.RemoveAt(PathStr.Len() - 1, EAllowShrinking::No);
			Path = *PathStr;

			if (!ParentPathToChildPaths.Contains(Path))
			{
				// Doesn't exist - nothing more to do
				return false;
			}
		}
		else
		{
			// Doesn't exist - nothing more to do
			return false;
		}
	}

	// We also need to gather up and remove any children of this path
	TSet<FName> SubPathsToRemove;
	GetSubPaths(Path, SubPathsToRemove, /*bRecurse=*/true);

	// Sort the sub-paths by length, longest -> shortest, so that children are notified before their parents
	SubPathsToRemove.Sort([](FName SubPathOne, FName SubPathTwo)
	{
		return SubPathOne.Compare(SubPathTwo) > 0;
	});

	// Simply remove sub-paths from both maps
	for (const FName& SubPathToRemove : SubPathsToRemove)
	{
		ParentPathToChildPaths.Remove(SubPathToRemove);
		ChildPathToParentPath.Remove(SubPathToRemove);
		OnPathRemoved(SubPathToRemove);
	}

	// We also need to remove ourself from our parent list before removing ourself from the maps
	{
		const FName* ParentPathPtr = ChildPathToParentPath.Find(Path);
		if (ParentPathPtr)
		{
			TSet<FName>& ChildPaths = ParentPathToChildPaths.FindChecked(*ParentPathPtr);
			ChildPaths.Remove(Path);
		}
	}
	
	VirtualToInternalMounts.Remove(Path);
	ParentPathToChildPaths.Remove(Path);
	ChildPathToParentPath.Remove(Path);
	OnPathRemoved(Path);

	return true;
}

bool FContentBrowserVirtualPathTree::PathExists(FName Path, bool& bIsFullyVirtual) const
{
	if (Path.IsNone())
	{
		bIsFullyVirtual = false;
		return false;
	}

	const TSet<FName>* ChildPathsPtr = ParentPathToChildPaths.Find(Path);
	if (!ChildPathsPtr)
	{
		// Paths are cached without their trailing slash, so if the given path has a trailing slash, test it again now as it may already be cached
		// We do this after the initial map test as: a) Most paths are well formed, b) This avoids string allocations until we know we need them
		FString PathStr = Path.ToString();
		if (PathStr[PathStr.Len() - 1] == '/')
		{
			PathStr.RemoveAt(PathStr.Len() - 1, EAllowShrinking::No);
			Path = *PathStr;

			ChildPathsPtr = ParentPathToChildPaths.Find(Path);
		}
	}

	bIsFullyVirtual = ChildPathsPtr && ChildPathsPtr->Num() > 0;

	return ChildPathsPtr != nullptr;
}

bool FContentBrowserVirtualPathTree::GetAllPaths(TSet<FName>& OutPaths) const
{
	OutPaths.Reset();
	EnumerateAllPaths([&OutPaths](FName Path)
	{
		OutPaths.Emplace(Path);
		return true;
	});
	return OutPaths.Num() > 0;
}

void FContentBrowserVirtualPathTree::EnumerateAllPaths(TFunctionRef<bool(FName)> Callback) const
{
	for (const auto& PathPair : ParentPathToChildPaths)
	{
		if (!Callback(PathPair.Key))
		{
			return;
		}
	}
}

bool FContentBrowserVirtualPathTree::GetSubPaths(FName BasePath, TSet<FName>& OutPaths, bool bRecurse) const
{
	const int32 OutPathsOriginalNum = OutPaths.Num();
	return EnumerateSubPaths(BasePath, [&OutPaths](FName Path, FName InternalPath)
	{
		OutPaths.Emplace(Path);
		return true;
	}, bRecurse) && OutPaths.Num() > OutPathsOriginalNum;
}

bool FContentBrowserVirtualPathTree::EnumerateSubPaths(FName BasePath, TFunctionRef<bool(FName, FName)> Callback, bool bRecurse) const
{
	if (BasePath.IsNone())
	{
		return false;
	}

	const TSet<FName>* ChildPathsPtr = ParentPathToChildPaths.Find(BasePath);
	if (!ChildPathsPtr)
	{
		// Paths are cached without their trailing slash, so if the given path has a trailing slash, test it again now as it may already be cached
		// We do this after the initial map test as: a) Most paths are well formed, b) This avoids string allocations until we know we need them
		FString BasePathStr = BasePath.ToString();
		if (BasePathStr[BasePathStr.Len() - 1] == '/')
		{
			BasePathStr.RemoveAt(BasePathStr.Len() - 1, EAllowShrinking::No);
			BasePath = *BasePathStr;

			ChildPathsPtr = ParentPathToChildPaths.Find(BasePath);
			if (!ChildPathsPtr)
			{
				return false;
			}
		}
		else
		{
			return false;
		}
	}

	for (const FName& ChildPath : *ChildPathsPtr)
	{
		check(ParentPathToChildPaths.Contains(ChildPath)); // This failing is an integrity violation as this entry lists a child that we don't know about

		const FName* InternalPath = VirtualToInternalMounts.Find(ChildPath);
		if (!Callback(ChildPath, InternalPath ? *InternalPath : NAME_None))
		{
			return true;
		}

		if (bRecurse)
		{
			EnumerateSubPaths(ChildPath, Callback, /*bRecurse=*/true);
		}
	}

	return true;
}

FName FContentBrowserVirtualPathTree::GetParentPath(FName Path) const
{
	if (Path.IsNone())
	{
		return NAME_None;
	}

	if (const FName* ParentPathPtr = ChildPathToParentPath.Find(Path))
	{
		return *ParentPathPtr;
	}

	return NAME_None;
}

int32 FContentBrowserVirtualPathTree::NumPaths() const
{
	return ParentPathToChildPaths.Num();
}

bool FContentBrowserVirtualPathTree::IsShowAllFolderEnabled() const
{
	return GetDefault<UContentBrowserSettings>()->bShowAllFolder;
}

const FString& FContentBrowserVirtualPathTree::GetAllFolderPrefix() const
{
	if (UContentBrowserDataSubsystem* ContentBrowserDataSubsystem = IContentBrowserDataModule::Get().GetSubsystem())
	{
		return ContentBrowserDataSubsystem->GetAllFolderPrefix();
	}

	static FString EmptyString;
	return EmptyString;
}

EContentBrowserPathType FContentBrowserVirtualPathTree::TryConvertVirtualPathToInternal(FStringView InPath, FStringBuilderBase& OutPath) const
{
	OutPath.Reset();

	// Special case
	static const FStringView RootPath(TEXT("/"));
	if (InPath.Equals(RootPath))
	{
		OutPath.Append(InPath);
		return EContentBrowserPathType::Virtual;
	}

	if (InPath.Len() == 0)
	{
		return EContentBrowserPathType::None;
	}

	// Walk path until reaches internal mount point or ends remaining as a virtual path
	bool bFailedPathExists = false;
	int32 BestMountPointSplitIndex = INDEX_NONE;
	int32 SplitIndex = 0;
	const TCHAR* PathStr = InPath.GetData();
	const TCHAR* PathStrEnd = PathStr + InPath.Len();
	const TCHAR* PathCharPtr = PathStr + 1;
	for (;;)
	{
		if (PathCharPtr >= PathStrEnd || *PathCharPtr == TEXT('/'))
		{
			const FName CheckPath(int32(PathCharPtr - PathStr), PathStr);
			bool bCheckPathIsFullyVirtual = false;
			if (PathExists(CheckPath, bCheckPathIsFullyVirtual))
			{
				if (!bCheckPathIsFullyVirtual)
				{
					BestMountPointSplitIndex = SplitIndex;
					break;
				}
				else
				{
					if (VirtualToInternalMounts.Contains(CheckPath))
					{
						// This virtual path has other virtual paths as children (which is why bCheckPathIsFullyVirtual returned true)
						// It's still a valid root to map against though, so keep track of it until we find a better match to use
						BestMountPointSplitIndex = SplitIndex;
					}
					SplitIndex = (int32)(PathCharPtr - PathStr);
				}
			}
			else
			{
				bFailedPathExists = true;
				break;
			}
		}

		if (PathCharPtr >= PathStrEnd)
		{
			// End of the string
			break;
		}

		++PathCharPtr;
	}

	if (SplitIndex > 0)
	{
		if (BestMountPointSplitIndex == INDEX_NONE)
		{
			if (bFailedPathExists)
			{
				return EContentBrowserPathType::None;
			}

			// Strip "/All" to turn fully virtual path into something that can survive being saved/loaded to .ini file for favorites, colors and other systems
			if (IsShowAllFolderEnabled())
			{
				FStringView InvariantPathView(InPath);
				InvariantPathView.RightChopInline(GetAllFolderPrefix().Len());
				if (InvariantPathView.Len() == 0)
				{
					OutPath.Append(TEXT("/"));
				}
				else
				{
					OutPath.Append(InvariantPathView);
				}
			}
			else
			{
				OutPath.Append(InPath);
			}

			return EContentBrowserPathType::Virtual;
		}

		FStringView InternalPathView(InPath);
		InternalPathView.RightChopInline(BestMountPointSplitIndex);
		OutPath.Append(InternalPathView);
		return EContentBrowserPathType::Internal;
	}
	else if (BestMountPointSplitIndex != INDEX_NONE)
	{
		OutPath.Append(InPath);
		return EContentBrowserPathType::Internal;
	}

	return EContentBrowserPathType::None;
}

EContentBrowserPathType FContentBrowserVirtualPathTree::TryConvertVirtualPathToInternal(FStringView InPath, FString& OutPath) const
{
	FNameBuilder OutPathBuilder;
	const EContentBrowserPathType ConvertedType = TryConvertVirtualPathToInternal(InPath, OutPathBuilder);
	OutPath = FString(FStringView(OutPathBuilder));
	return ConvertedType;
}

EContentBrowserPathType FContentBrowserVirtualPathTree::TryConvertVirtualPathToInternal(FName InPath, FName& OutPath) const
{
	FNameBuilder OutPathBuilder;
	const EContentBrowserPathType ConvertedType = TryConvertVirtualPathToInternal(FNameBuilder(InPath), OutPathBuilder);
	OutPath = FName(FStringView(OutPathBuilder));
	return ConvertedType;
}
