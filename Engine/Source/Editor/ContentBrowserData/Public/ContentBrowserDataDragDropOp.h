// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserItem.h"
#include "CoreMinimal.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "HAL/Platform.h"
#include "Input/DragAndDrop.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

#define UE_API CONTENTBROWSERDATA_API

class UActorFactory;
struct FAssetData;

/**
 * Additional params for FContentBrowserDataDragDropOp to override the thumbnail used if dragging only folders
 */
struct FThumbnailOverrideParams
{
	FThumbnailOverrideParams() {}

	/** Folder brush name to use */
	FName FolderBrushName = NAME_None;

	/** Folder shadow brush name to use */
	FName FolderShadowBrushName = NAME_None;

	/** Override color for the final brush */
	FLinearColor FolderColorOverride = FLinearColor::Gray;
};

class FContentBrowserDataDragDropOp : public FAssetDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FContentBrowserDataDragDropOp, FAssetDragDropOp)

	static UE_API TSharedRef<FContentBrowserDataDragDropOp> New(TArrayView<const FContentBrowserItem> InDraggedItems, FThumbnailOverrideParams InThumbnailOverrideParams = FThumbnailOverrideParams());

	static UE_API TSharedRef<FContentBrowserDataDragDropOp> Legacy_New(TArrayView<const FAssetData> InAssetData, TArrayView<const FString> InAssetPaths = TArrayView<const FString>(), UActorFactory* InActorFactory = nullptr);

	const TArray<FContentBrowserItem>& GetDraggedItems() const
	{
		return DraggedItems;
	}

	const TArray<FContentBrowserItem>& GetDraggedFiles() const
	{
		return DraggedFiles;
	}

	const TArray<FContentBrowserItem>& GetDraggedFolders() const
	{
		return DraggedFolders;
	}

private:
	UE_API void Init(TArrayView<const FContentBrowserItem> InDraggedItems);
	UE_API void LegacyInit(TArrayView<const FAssetData> InAssetData, TArrayView<const FString> InAssetPaths, UActorFactory* ActorFactory);
	UE_API virtual void InitThumbnail() override;

	UE_API virtual bool HasFiles() const override;
	UE_API virtual bool HasFolders() const override;

	UE_API virtual int32 GetTotalCount() const override;
	UE_API virtual FText GetFirstItemText() const override;

	/** Return whether or not it should override the thumbnail widget */
	UE_API bool ShouldOverrideThumbnailWidget() const;

	/** Return the folder widget or nullptr if the BrushName is not valid */
	UE_API TSharedPtr<SWidget> GetFolderWidget() const;

	/** Return the folder widget for DragAndDrop or nullptr if there is no basic folder widget  */
	UE_API TSharedPtr<SWidget> GetFolderWidgetDragAndDrop() const;

private:
	FName FolderBrushName;
	FName FolderShadowBrushName;
	FLinearColor FolderColorOverride = FLinearColor::Gray;
	TArray<FContentBrowserItem> DraggedItems;
	TArray<FContentBrowserItem> DraggedFiles;
	TArray<FContentBrowserItem> DraggedFolders;
};

#undef UE_API
