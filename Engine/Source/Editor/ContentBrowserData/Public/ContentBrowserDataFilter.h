// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CollectionManagerTypes.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/SortedMap.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/OptionalFwd.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/StructOnScope.h"

#include "ContentBrowserDataFilter.generated.h"

#define UE_API CONTENTBROWSERDATA_API

class FNamePermissionList;
class FPathPermissionList;
class UContentBrowserDataSource;
class UContentBrowserDataSubsystem;

/**
 * Flags controlling which item types should be included
 * Generally matches EContentBrowserItemFlags::Type_MASK
 */
UENUM(Flags)
enum class EContentBrowserItemTypeFilter : uint8
{
	IncludeNone = 0,
	IncludeFolders = 1<<0,
	IncludeFiles = 1<<1,
	IncludeAll = IncludeFolders | IncludeFiles,
};
ENUM_CLASS_FLAGS(EContentBrowserItemTypeFilter);

/**
 * Flags controlling which item categories should be included
 * Generally matches EContentBrowserItemFlags::Category_MASK
 */
UENUM(Flags)
enum class EContentBrowserItemCategoryFilter : uint8
{
	IncludeNone = 0,
	IncludeAssets = 1<<0,
	IncludeClasses = 1<<1,
	IncludeCollections = 1<<2,
	IncludeRedirectors = 1 << 3,
	IncludeMisc = 1 << 4,
	IncludeAll = IncludeAssets | IncludeClasses | IncludeCollections | IncludeRedirectors | IncludeMisc,
};
ENUM_CLASS_FLAGS(EContentBrowserItemCategoryFilter);

/** Flags controlling which item attributes should be included */
UENUM(Flags)
enum class EContentBrowserItemAttributeFilter : uint8
{
	IncludeNone = 0,
	IncludeProject = 1<<0,
	IncludeEngine = 1<<1,
	IncludePlugins = 1<<2,
	IncludeDeveloper = 1<<3,
	IncludeLocalized = 1<<4,
	IncludeAll = IncludeProject | IncludeEngine | IncludePlugins | IncludeDeveloper | IncludeLocalized,
};
ENUM_CLASS_FLAGS(EContentBrowserItemAttributeFilter);

/*
 * Structure used to filter folders by their path
 * @note All implementations of this filter should copy all dependent data on creation such that all calls to HideFolderIfEmpty for a given path return the same value for a instance of the implementation.
 */
class IContentBrowserHideFolderIfEmptyFilter
{
public:
	IContentBrowserHideFolderIfEmptyFilter() = default;
	virtual ~IContentBrowserHideFolderIfEmptyFilter() = default;

	virtual bool HideFolderIfEmpty(FName Path, FStringView PathString) const = 0;
};

/**
 * A list of typed filter structs and their associated data.
 * This allows systems to add new filter types that the core Content Browser data module doesn't know about.
 */
struct FContentBrowserDataFilterList
{
public:
	/** Constructor */
	FContentBrowserDataFilterList() = default;

	/** Copy support */
	UE_API FContentBrowserDataFilterList(const FContentBrowserDataFilterList& InOther);
	UE_API FContentBrowserDataFilterList& operator=(const FContentBrowserDataFilterList& InOther);

	/** Move support */
	FContentBrowserDataFilterList(FContentBrowserDataFilterList&&) = default;
	FContentBrowserDataFilterList& operator=(FContentBrowserDataFilterList&&) = default;

	/** Find the filter associated with the given type, or add a default instance if it doesn't exist in the list */
	template <typename T>
	T& FindOrAddFilter()
	{
		return *static_cast<T*>(FindOrAddFilter(TBaseStructure<T>::Get()));
	}
	UE_API void* FindOrAddFilter(const UScriptStruct* InFilterType);

	/** Set the filter associated with the given type, replacing any instance of this type that may exist in the list */
	template <typename T>
	void SetFilter(const T& InFilter)
	{
		SetFilter(TBaseStructure<T>::Get(), &InFilter);
	}
	UE_API void SetFilter(const UScriptStruct* InFilterType, const void* InFilterData);

	/** Find the filter associated with the given type, if it exists in the list */
	template <typename T>
	const T* FindFilter() const
	{
		return static_cast<const T*>(FindFilter(TBaseStructure<T>::Get()));
	}
	UE_API const void* FindFilter(const UScriptStruct* InFilterType) const;

	/** Find the filter associated with the given type, if it exists in the list */
	template <typename T>
	T* FindMutableFilter()
	{
		return const_cast<T*>(FindMutableFilter<T>());
	}

	/** Get the filter associated with the given type, asserting if it doesn't exist in the list */
	template <typename T>
	const T& GetFilter() const
	{
		const T* FilterData = FindFilter<T>();
		check(FilterData);
		return *FilterData;
	}

	/** Get the filter associated with the given type, asserting if it doesn't exist in the list */
	template <typename T>
	T& GetMutableFilter()
	{
		return const_cast<T&>(GetFilter<T>());
	}

	/** Remove the filter associated with the given type */
	template <typename T>
	void RemoveFilter()
	{
		RemoveFilter(TBaseStructure<T>::Get());
	}
	UE_API void RemoveFilter(const UScriptStruct* InFilterType);

	/** Remove all filters in the list */
	UE_API void ClearFilters();
	
	UE_API TArray<const UScriptStruct*> GetFilterTypes() const;

private:
	/** Set the contents of this list to be a deep copy of the contents of the other list */
	UE_API void SetTo(const FContentBrowserDataFilterList& InOther);

	/** Array of typed filter structs */
	TArray<FStructOnScope> TypedFilters;
};

struct FContentBrowserDataFilterCacheID
{
public:
	operator bool() const
	{
		return IsSet();
	};

	bool IsSet() const
	{
		return ID != INDEX_NONE;
	}

private:
	friend struct FContentBrowserDataFilterCacheIDOwner;

	int64 ID = INDEX_NONE;

	friend uint32 GetTypeHash(const FContentBrowserDataFilterCacheID& CacheID)
	{
		return GetTypeHash(CacheID.ID);
	}
};

/**
 * A filter used to control what is returned from Content Browser data queries.
 * @note The compiled version of this, FContentBrowserDataCompiledFilter, is produced via UContentBrowserDataSubsystem::CompileFilter.
 */
USTRUCT(BlueprintType)
struct FContentBrowserDataFilter
{
	GENERATED_BODY()

public:
	/** Whether we should include sub-paths in this query */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="ContentBrowser")
	bool bRecursivePaths = false;

	/** Flags controlling which item types should be included in this query */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="ContentBrowser")
	EContentBrowserItemTypeFilter ItemTypeFilter = EContentBrowserItemTypeFilter::IncludeAll;

	/** Flags controlling which item categories should be included in this query */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="ContentBrowser")
	EContentBrowserItemCategoryFilter ItemCategoryFilter = EContentBrowserItemCategoryFilter::IncludeAll;

	/** Flags controlling which item attributes should be included in this query */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="ContentBrowser")
	EContentBrowserItemAttributeFilter ItemAttributeFilter = EContentBrowserItemAttributeFilter::IncludeAll;

	/** A list of extra filter structs to be interpreted by the Content Browser data sources */
	FContentBrowserDataFilterList ExtraFilters;

	/** An optional id used by the data sources to cache and reuse some data when compiling the filter(s) */
	FContentBrowserDataFilterCacheID CacheID;
};

/**
 * A filter used to control what is returned from Content Browser data queries.
 * @note The source version of this, FContentBrowserDataFilter, is used with UContentBrowserDataSubsystem::CompileFilter to produce a compiled filter.
 */
struct FContentBrowserDataCompiledFilter
{
	/** Flags controlling which item types should be included in this query */
	EContentBrowserItemTypeFilter ItemTypeFilter = EContentBrowserItemTypeFilter::IncludeAll;

	/** Flags controlling which item categories should be included in this query */
	EContentBrowserItemCategoryFilter ItemCategoryFilter = EContentBrowserItemCategoryFilter::IncludeAll;

	/** Flags controlling which item attributes should be included in this query */
	EContentBrowserItemAttributeFilter ItemAttributeFilter = EContentBrowserItemAttributeFilter::IncludeAll;

	/** Per data-source compiled filter structs - typically optimized for both search queries and per-item queries */
	TSortedMap<const UContentBrowserDataSource*, FContentBrowserDataFilterList> CompiledFilters;
};

/**
 * Data used to filter object instances by their name and tags.
 * @note This will typically limit your query to returning assets.
 */
USTRUCT(BlueprintType)
struct FContentBrowserDataObjectFilter
{
	GENERATED_BODY()

public:
	/** Array of object names that should be included in this query */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="ContentBrowser")
	TArray<FName> ObjectNamesToInclude;

	/** Array of object names that should be excluded from this query */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="ContentBrowser")
	TArray<FName> ObjectNamesToExclude;

	/** Whether we should only include on-disk objects (ignoring those that exist only in memory) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="ContentBrowser")
	bool bOnDiskObjectsOnly = false;

	/** Map of object tags (with optional values) that should be included in this query */
	TMultiMap<FName, TOptional<FString>> TagsAndValuesToInclude;

	/** Map of object tags (with optional values) that should be excluded from this query */
	TMultiMap<FName, TOptional<FString>> TagsAndValuesToExclude;
};

/**
 * Data used to filter object instances by their package.
 * @note This will typically limit your query to returning assets.
 */
USTRUCT(BlueprintType)
struct FContentBrowserDataPackageFilter
{
	GENERATED_BODY()

public:
	/** Array of package names that should be included in this query */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="ContentBrowser")
	TArray<FName> PackageNamesToInclude;

	/** Array of package names that should be excluded from this query */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="ContentBrowser")
	TArray<FName> PackageNamesToExclude;

	/** Array of package paths that should be included in this query */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="ContentBrowser")
	TArray<FName> PackagePathsToInclude;

	/** Array of package paths that should be excluded from this query */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="ContentBrowser")
	TArray<FName> PackagePathsToExclude;

	/** Whether we should include inclusive package sub-paths in this query */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="ContentBrowser")
	bool bRecursivePackagePathsToInclude = false;

	/** Whether we should include exclusive package sub-paths in this query */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="ContentBrowser")
	bool bRecursivePackagePathsToExclude = false;

	/** Optional set of additional path filtering */
	TSharedPtr<FPathPermissionList> PathPermissionList;
};

/**
 * Data used to filter object instances by their class.
 * @note This will typically limit your query to returning assets.
 */
USTRUCT(BlueprintType)
struct FContentBrowserDataClassFilter
{
	GENERATED_BODY()

public:
	/** Array of class names that should be included in this query */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="ContentBrowser")
	TArray<FString> ClassNamesToInclude;

	/** Array of class names that should be excluded from this query */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="ContentBrowser")
	TArray<FString> ClassNamesToExclude;

	/** Whether we should include inclusive sub-classes in this query */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="ContentBrowser")
	bool bRecursiveClassNamesToInclude = false;

	/** Whether we should include exclusive sub-classes in this query */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="ContentBrowser")
	bool bRecursiveClassNamesToExclude = false;

	/** Optional set of additional class filtering */
	TSharedPtr<FPathPermissionList> ClassPermissionList;
};

/**
 * Data used to filter items by their collection.
 * @note This will typically limit your query to items that support being inside a collection.
 */
USTRUCT(BlueprintType)
struct FContentBrowserDataCollectionFilter
{
	GENERATED_BODY()

public:
	FContentBrowserDataCollectionFilter() = default;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	FContentBrowserDataCollectionFilter(const FContentBrowserDataCollectionFilter&) = default;
	FContentBrowserDataCollectionFilter(FContentBrowserDataCollectionFilter&&) = default;

	FContentBrowserDataCollectionFilter& operator=(const FContentBrowserDataCollectionFilter&) = default;
	FContentBrowserDataCollectionFilter& operator=(FContentBrowserDataCollectionFilter&&) = default;

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Array of collections to include in this query */
	TArray<FCollectionRef> Collections;

	UE_DEPRECATED(5.6, "Use Collections instead.")
	TArray<FCollectionNameType> SelectedCollections;

	/** Whether we should include child collections in this query */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="ContentBrowser")
	bool bIncludeChildCollections = false;
};

/*
 * Structure used to optionally filter folders by their broad contents
 */
struct FContentBrowserFolderContentsFilter
{
public:
	EContentBrowserItemTypeFilter ItemTypeFilter = EContentBrowserItemTypeFilter::IncludeAll;

	EContentBrowserItemCategoryFilter ItemCategoryFilter = EContentBrowserItemCategoryFilter::IncludeAll;

	EContentBrowserItemAttributeFilter ItemAttributeFilter = EContentBrowserItemAttributeFilter::IncludeAll;

	/**
	* Used to filter out empty folders. All empty folders will get filtered out if this TSharedPtr is null.
	* Intended for content folders that always exist but do not generally contain assets, such as /Game/Collections
	*/
	TSharedPtr<IContentBrowserHideFolderIfEmptyFilter> HideFolderIfEmptyFilter;
};

/**
 * Data used to tell the content browser to show the item that doesn't pass the class permission list as a unsupported asset
 * @note This will restrict user interaction with those asset in the content browser and only affect the asset in the folders specified in the permission list
 */
USTRUCT()
struct FContentBrowserDataUnsupportedClassFilter
{
	GENERATED_BODY()

public:
	TSharedPtr<FPathPermissionList> ClassPermissionList;

	TSharedPtr<FPathPermissionList> FolderPermissionList;
};

/**
 * ID used by the data sources to cache some data between the filter compilations
 * How use the filter compilation cache.
 * 1) Initialize the id by using the UContentBrowserDataSubsystem once.
 * 2) When compiling the filters pass the Cache ID Owner to the cacheID of the ContentBrowserDataFilter.
 * 3) When the filter settings change call RemoveUnusedCachedData to clean the cache and to remove the potential invalid data.
 */
struct FContentBrowserDataFilterCacheIDOwner
{
public:
	operator FContentBrowserDataFilterCacheID() const
	{
		FContentBrowserDataFilterCacheID CacheID;
		CacheID.ID = ID;
		return CacheID;
	}

	FContentBrowserDataFilterCacheIDOwner() = default;

	FContentBrowserDataFilterCacheIDOwner(FContentBrowserDataFilterCacheIDOwner&& Other)
		: ID(Other.ID)
		, DataSource(MoveTemp(Other.DataSource))
	{
		Other.ID = INDEX_NONE;
		Other.DataSource.Reset();
	}

	FContentBrowserDataFilterCacheIDOwner& operator=(FContentBrowserDataFilterCacheIDOwner&& Other)
	{
		ID = Other.ID;
		Other.ID = INDEX_NONE;
		Other.DataSource = MoveTemp(Other.DataSource);
		Other.DataSource.Reset();

		return *this;
	}

	~FContentBrowserDataFilterCacheIDOwner()
	{
		ClearCachedData();
	}

	FContentBrowserDataFilterCacheIDOwner(const FContentBrowserDataFilterCacheIDOwner&) = delete;
	FContentBrowserDataFilterCacheIDOwner& operator=(const FContentBrowserDataFilterCacheIDOwner&) = delete;

	UE_API void Initialaze(UContentBrowserDataSubsystem* InContentBrowserDataSubsystem);

	UE_API void RemoveUnusedCachedData(TArrayView<const FName> InVirtualPathsInUse, const FContentBrowserDataFilter& DataFilter) const;

	UE_API void ClearCachedData() const;

	UE_API void Reset();

private:
	friend UContentBrowserDataSubsystem;

	int64 ID = INDEX_NONE;
	TWeakObjectPtr<UContentBrowserDataSubsystem> DataSource;
};

#undef UE_API
