// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "ContentBrowserDataMenuContexts.h"
#include "ContentBrowserItemData.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "UObject/ObjectMacros.h"

#include "ContentBrowserDataLegacyBridge.generated.h"

class FName;
class UClass;
class UFactory;
struct FARFilter;
struct FAssetData;

/** Called to add extra asset data to the asset view, to display virtual assets. These get treated similar to Class assets */
DECLARE_DELEGATE_TwoParams(FOnGetCustomSourceAssets, const FARFilter& /*SourceFilter*/, TArray<FAssetData>& /*AddedAssets*/);

/** Called to begin user controlled asset creation via the asset data source (if available) */
DECLARE_DELEGATE_FiveParams(FOnCreateNewAsset, const FName /*DefaultAssetName*/, const FName /*PackagePath*/, UClass* /*AssetClass*/, UFactory* /*Factory*/, UContentBrowserDataMenuContext_AddNewMenu::FOnBeginItemCreation /*OnBeginItemCreation*/);

/** Filter data used to provide legacy information to the asset data source */
USTRUCT()
struct FContentBrowserDataLegacyFilter
{
	GENERATED_BODY()

public:
	FOnGetCustomSourceAssets OnGetCustomSourceAssets;
};

namespace ContentBrowserDataLegacyBridge
{

/** Called to begin user controlled asset creation via the asset data source (if available) */
CONTENTBROWSERDATA_API FOnCreateNewAsset& OnCreateNewAsset();

}
