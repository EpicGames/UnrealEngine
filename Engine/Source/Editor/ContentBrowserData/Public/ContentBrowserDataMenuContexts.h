// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ContentBrowserItem.h"
#include "CollectionManagerTypes.h"
#include "ContentBrowserDataMenuContexts.generated.h"

class SWidget;
struct FContentBrowserInstanceConfig;

UENUM()
enum class EContentBrowserDataMenuContext_AddNewMenuDomain : uint8
{
	Toolbar,
	AssetView,
	PathView,
};

UCLASS(MinimalAPI)
class UContentBrowserDataMenuContext_AddNewMenu : public UObject
{
	GENERATED_BODY()

public:
	DECLARE_DELEGATE_OneParam(FOnBeginItemCreation, const FContentBrowserItemDataTemporaryContext&);

	UPROPERTY(BlueprintReadOnly, Category = ContentBrowser)
	TArray<FName> SelectedPaths;

	// At least one of the selected paths maps to a mounted content root
	UPROPERTY(BlueprintReadOnly, Category = ContentBrowser)
	bool bContainsValidPackagePath = false;

	UPROPERTY(BlueprintReadOnly, Category = ContentBrowser)
	bool bCanBeModified = true;

	UPROPERTY(BlueprintReadOnly, Category = ContentBrowser)
	EContentBrowserDataMenuContext_AddNewMenuDomain OwnerDomain = EContentBrowserDataMenuContext_AddNewMenuDomain::Toolbar;

	FOnBeginItemCreation OnBeginItemCreation;

	const FContentBrowserInstanceConfig* OwningInstanceConfig;
};

UCLASS(MinimalAPI)
class UContentBrowserDataMenuContext_FolderMenu : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadOnly, Category=ContentBrowser)
	TArray<FContentBrowserItem> SelectedItems;

	UPROPERTY(BlueprintReadOnly, Category = ContentBrowser)
	bool bCanBeModified = true;

	TWeakPtr<SWidget> ParentWidget;
};

UCLASS(MinimalAPI)
class UContentBrowserDataMenuContext_FileMenu : public UObject
{
	GENERATED_BODY()

public:
	DECLARE_DELEGATE_OneParam(FOnShowInPathsView, TArrayView<const FContentBrowserItem>);

	DECLARE_DELEGATE(FOnRefreshView);

	UPROPERTY(BlueprintReadOnly, Category = ContentBrowser)
	TArray<FContentBrowserItem> SelectedItems;

	TArray<FCollectionRef> CollectionSources;

	UE_DEPRECATED(5.6, "Use CollectionSources instead.")
	TArray<FCollectionNameType> SelectedCollections;

	UPROPERTY(BlueprintReadOnly, Category = ContentBrowser)
	bool bCanBeModified = true;

	UPROPERTY(BlueprintReadOnly, Category = ContentBrowser)
	bool bCanView = true;

	UPROPERTY(BlueprintReadOnly, Category = ContentBrowser)
	bool bHasCookedPackages = false;

	UPROPERTY(BlueprintReadOnly, Category = ContentBrowser)
	bool bContainsUnsupportedAssets = true;

	TWeakPtr<SWidget> ParentWidget;

	FOnShowInPathsView OnShowInPathsView;

	FOnRefreshView OnRefreshView;
};

UCLASS(MinimalAPI)
class UContentBrowserDataMenuContext_DragDropMenu : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadOnly, Category = ContentBrowser)
	FContentBrowserItem DropTargetItem;

	UPROPERTY(BlueprintReadOnly, Category = ContentBrowser)
	TArray<FContentBrowserItem> DraggedItems;

	UPROPERTY(BlueprintReadOnly, Category = ContentBrowser)
	bool bCanMove = true;

	UPROPERTY(BlueprintReadOnly, Category = ContentBrowser)
	bool bCanCopy = true;

	TWeakPtr<SWidget> ParentWidget;
};
