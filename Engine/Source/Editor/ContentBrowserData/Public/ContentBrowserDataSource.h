// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDataFilter.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserItemData.h"
#include "ContentBrowserVirtualPathTree.h"
#include "CoreMinimal.h"
#include "Features/IModularFeature.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#include "ContentBrowserDataSource.generated.h"

#define UE_API CONTENTBROWSERDATA_API

class FAssetThumbnail;
class FDragDropEvent;
class FDragDropOperation;
class FText;
struct FAssetData;
struct FContentBrowserItemPath;
template <typename FuncType> class TFunctionRef;

namespace ContentBrowserItemAttributes
{
	/**
	 * Attribute key that can be used to query the internal type name of an item.
	 * Type: FName.
	 */
	const FName ItemTypeName = "ItemTypeName";

	/**
	 * Attribute key that can be used to query the internal type display name of an item.
	 * Type: FText.
	 */
	const FName ItemTypeDisplayName = "ItemTypeDisplayName";

	/**
	 * Attribute key that can be used to query the internal description of an item.
	 * Type: FText.
	 */
	const FName ItemDescription = "ItemDescription";

	/**
	 * Attribute key that can be used to query the internal disk size of an item.
	 * Type: int64.
	 */
	const FName ItemDiskSize = "ItemDiskSize";

	/**
	 * Attribute key that can be used to query if the item has virtualized data or not.
	 * Type: bool.
	 */
	const FName VirtualizedData = "HasVirtualizedData";

	/**
	 * Attribute key that can be used to query whether the given item is considered to be developer content.
	 * Type: bool.
	 */
	const FName ItemIsDeveloperContent = "ItemIsDeveloperContent";

	/**
	 * Attribute key that can be used to query whether the given item is considered to be localized content.
	 * Type: bool.
	 */
	const FName ItemIsLocalizedContent = "ItemIsLocalizedContent";

	/**
	 * Attribute key that can be used to query whether the given item is considered to be engine content (including engine plugin content).
	 * Type: bool.
	 */
	const FName ItemIsEngineContent = "ItemIsEngineContent";

	/**
	 * Attribute key that can be used to query whether the given item is considered to be project content (including project plugin content).
	 * Type: bool.
	 */
	const FName ItemIsProjectContent = "ItemIsProjectContent";

	/**
	 * Attribute key that can be used to query whether the given item is considered to be plugin content.
	 * Type: bool.
	 */
	const FName ItemIsPluginContent = "ItemIsPluginContent";

	/**
	 * Attribute key that can be used to query whether the given item is considered to be a custom virtual folder for organizational purposes
	 * that should be presented with a different folder icon.
	 * Type: bool.
	 */
	const FName ItemIsCustomVirtualFolder = "ItemIsCustomVirtualFolder";

	/**
	 * Attribute key that can be used to query the display color of an item.
	 * Type: FLinearColor (as FString).
	 */
	const FName ItemColor = "ItemColor";

	/**
	 * Attribute key that can be used to query the item's brush to use for an open folder in an asset tree.
	 * Type: Brush name (as FName).
	 */
	const FName ItemAssetTreeFolderOpenBrush = "ItemAssetTreeFolderOpenBrush";

	/**
	 * Attribute key that can be used to query the item's brush to use for a closed folder in an asset tree.
	 * Type: Brush name (as FName).
	 */
	const FName ItemAssetTreeFolderClosedBrush = "ItemAssetTreeFolderClosedBrush";

	/**
	 * Attribute key that can be used to query the item's brush to use for a folder in a list view.
	 * Type: Brush name (as FName).
	 */
	const FName ItemListViewFolderBrush = "ItemListViewFolderBrush";
}

/**
 * A common implementation of a "do nothing" data source for the Content Browser.
 * You should derive from this type to create new data sources for the Content Browser, overriding any required functionality and validation logic.
 *
 * Data sources create and operate on FContentBrowserItemData instances that represent the folders and files within each data source.
 * FContentBrowserItemData itself is a concrete type, so extensibility is handled via the IContentBrowserItemDataPayload interface, which can be 
 * used to store any data source defined payload data that is required to operate on the underlying thing that the item represents.
 *
 * This is the only API you need to implement to create a data source, as each FContentBrowserItemData instance knows which data source owns it, 
 * and uses that information to pass itself back into the correct data source instance when asked to perform actions or validation.
 * In that sense you can think of this like a C API, where the data source returns an opaque object that is later passed back into the data source 
 * functions so that they can interpret the opaque object and provide functionality for it.
 */
UCLASS(MinimalAPI, Abstract, Transient)
class UContentBrowserDataSource : public UObject, public IModularFeature
{
	GENERATED_BODY()

public:
	//~ UObject interface
	UE_API virtual void BeginDestroy() override final;

	/**
	 * Get the name used when registering data source modular feature instances for use with the Content Browser Data Subsystem.
	 */
	static UE_API FName GetModularFeatureTypeName();

	/**
	 * Register this data source instance for use with the Content Browser Data Subsystem.
	 * @note This does not activate the data source. @see the EnabledDataSources array in the Content Browser Data Subsystem for more information on activation.
	 */
	UE_API void RegisterDataSource();

	/**
	 * Unregister this data source instance from the Content Browser Data Subsystem.
	 */
	UE_API void UnregisterDataSource();

	/**
	 * Set the data sink that can be used to communicate with the Content Browser Data Subsystem.
	 * This is set to a valid instance when the data source is activated, and set to null when it is deactivated.
	 */
	UE_API void SetDataSink(IContentBrowserItemDataSink* InDataSink);

	/**
	 * True if this data source is currently initialized.
	 * A data source should be initialized before it is registered.
	 */
	UE_API bool IsInitialized() const;

	/**
	 * Initialize this data source instance, optionally registering it once the initialization has finished (@see RegisterDataSource).
	 * @note This function is non-virtual because its signature may change on derived types, and so should be called directly on an instance of the correct type.
	 *
	 * @param InAutoRegister True to automatically register this instance once initialization has finished.
	 */
	UE_API void Initialize(const bool InAutoRegister = true);

	/**
	 * Shutdown this data source instance.
	 * @note This is called during the BeginDestroy phase of object destruction.
	 */
	UE_API virtual void Shutdown();

	/**
	 * Tick this data source instance.
	 * @note Called once every 0.1 seconds, prior to the Content Browser Data Subsystem emitting any pending item update notifications.
	 */
	UE_API virtual void Tick(const float InDeltaTime);
	/**
	 * Test whether the given virtual path is under the virtual mount root that was passed to Initialize.
	 * @note This also returns true if the given virtual path *is* the virtual mount root.
	 */
	UE_API bool IsVirtualPathUnderMountRoot(const FName InPath) const;

	/**
	 * Given a path and a data filter, produce an optimized filter that can be used to efficiently enumerate items that match it, and also query whether an item would pass it.
	 * @note This function *must not* block waiting on content discovery! It should use the current state as known at this point in time.
	 * @note A compiled filter should be short-lived (no more than 1 frame).
	 * @see EnumerateItemsMatchingFilter and DoesItemPassFilter.
	 *
	 * @param InPath The virtual path to search for items under.
	 * @param InFilter Rules describing how items should be filtered.
	 * @param OutCompiledFilter The compiled filter instance to fill with the result.
	 */
	UE_API virtual void CompileFilter(const FName InPath, const FContentBrowserDataFilter& InFilter, FContentBrowserDataCompiledFilter& OutCompiledFilter);

	/**
	 * Enumerate items that match the given compiled filter, invoking the callback for each matching item.
	 * @note This function *must not* block waiting on content discovery! It should use the current state as known at this point in time.
	 * @see CompileFilter.
	 *
	 * @param InFilter The compiled filter used to find matching items.
	 * @param InCallback The function to invoke for each matching item (return true to continue enumeration).
	 * @param InSink Parameter provided to receive output as either a callback or an array.
	 */
	UE_API virtual void EnumerateItemsMatchingFilter(const FContentBrowserDataCompiledFilter& InFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback);
	UE_API virtual void EnumerateItemsMatchingFilter(const FContentBrowserDataCompiledFilter& InFilter, const TGetOrEnumerateSink<FContentBrowserItemData>& InSink);

	/**
	 * Enumerate items that have the given virtual path, optionally filtering by type, and invoking the callback for each matching item.
	 * @note This function *must not* block waiting on content discovery! It should use the current state as known at this point in time.
	 *
	 * @param InPath The virtual path to find items for.
	 * @param InItemTypeFilter The types of items we want to find.
	 * @param InCallback The function to invoke for each matching item (return true to continue enumeration).
	 * @param InSink Parameter provided to receive output as either a callback or an array.
	 */
	UE_API virtual void EnumerateItemsAtPath(const FName InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback);
	UE_API virtual void EnumerateItemsAtPath(const FName InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, const TGetOrEnumerateSink<FContentBrowserItemData>& InSink);
	
	/**
	 * Enumerate the items (folders and/or files) that exist at the given content browser paths.
	 * @note Multiple items may have the same virtual path if they are different types, or come from different data sources.
	 *
	 * @param InPaths The paths to search for
	 * @param InItemTypeFilter The types of items we want to find.
	 * @param InCallback The function to invoke for each matching item (return true to continue enumeration).
	 */
	UE_API virtual bool EnumerateItemsAtPaths(const TArrayView<FContentBrowserItemPath> InPaths, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback);

	/**
	 * Enumerate items at the given user provided path, optionally filtering by type, and invoking the callback for each matching item.
	 * The user provided path can be any path format the data source supports other than a virtual path. The data source will deduce the path type.
	 * @note This function *must not* block waiting on content discovery! It should use the current state as known at this point in time.
	 *
	 * @param InPath The user provided path to find items for.
	 * @param InItemTypeFilter The types of items we want to find.
	 * @param InCallback The function to invoke for each matching item (return true to continue enumeration).
	 * @param InSink Parameter provided to receive output as either a callback or an array.
	 */
	UE_API virtual void EnumerateItemsAtUserProvidedPath(const FStringView InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback);
	UE_API virtual void EnumerateItemsAtUserProvidedPath(const FStringView InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, const TGetOrEnumerateSink<FContentBrowserItemData>& InSink);

	/**
	 * Enumerate the items (files) that exist for the given objects.
	 * @note Multiple items may have the same virtual path if they are different types, or come from different data sources.
	 *
	 * @param InObjects The objects to enumerate
	 * @param InCallback The function to invoke for each matching item (return true to continue enumeration).
	 */
	UE_API virtual bool EnumerateItemsForObjects(const TArrayView<UObject*> InObjects, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback);

	/**
	 * Get a list of other paths that the data source may be using to represent a specific path
	 *
	 * @param InInternalPath The internal path (or object path) of an asset to get aliases for
	 * @return All alternative paths that represent the input path (not including the input path itself)
	 */
	UE_API virtual TArray<FContentBrowserItemPath> GetAliasesForPath(const FSoftObjectPath& InInternalPath) const;
	
	/**
	 * Get the source asset path of an alias.
	 * 
	 * @param InItemData The content browser item of an alias
	 * @param OutObjectPath The soft object path to fill with the result
	 * @return True if @code InItemData @endcode is an alias and a path was found, false otherwise
	 */
	UE_API virtual bool GetItemAliasedPath(const FContentBrowserItemData& InItemData, FSoftObjectPath& OutObjectPath) const;

	UE_DEPRECATED(5.1, "FNames containing full asset paths are deprecated. Use FSoftObjectPath instead.")
	UE_API TArray<FContentBrowserItemPath> GetAliasesForPath(FName InInternalPath) const;

	/**
	 * Query whether this data source instance is currently discovering content, and retrieve an optional status message that can be shown in the UI.
	 */
	UE_API virtual bool IsDiscoveringItems(FText* OutStatus = nullptr);

	/**
	 * If possible, attempt to prioritize content discovery for the given virtual path.
	 * @note This will be called in response to a user attempting to browse content under the given virtual path.
	 */
	UE_API virtual bool PrioritizeSearchPath(const FName InPath);

	/**
	 * Query whether the given virtual folder should be visible in the UI.
	 * @note This function must be able to answer the question quickly or not at all (and assume visible). It *must not* block doing something like a file system scan.
	 * @param Path Virtual path of folder e.g. /All/GameData/Stuff
	 * @param Flags Currently unused
	 * @param ContentsFilter Optional filter for limiting visibility to only folders with a certain type of contents based on desired view filtering. If this is unset, folders may be visible regardless of their contents or empty status.
	 */
	UE_API virtual bool IsFolderVisible(const FName Path, const EContentBrowserIsFolderVisibleFlags Flags, const FContentBrowserFolderContentsFilter& ContentsFilter);

	/*
	 * Query whether a folder can be created at the given virtual path, optionally providing error information if it cannot.
	 *
	 * @param InPath The virtual path of the folder that is being queried.
	 * @param OutErrorMessage Optional error message to fill on failure.
	 *
	 * @return True if the folder can be created, false otherwise.
	 */
	UE_API virtual bool CanCreateFolder(const FName InPath, FText* OutErrorMsg);

	/*
	 * Attempt to begin the process of asynchronously creating a folder at the given virtual path, populating a temporary item that can be finalized or canceled by the user.
	 *
	 * @param InPath The initial virtual path of the folder that is being created.
	 * @param OutPendingItem Temporary item context to fill with information about the pending folder item.
	 *
	 * @return True if the pending folder was created, false otherwise.
	 */
	UE_API virtual bool CreateFolder(const FName InPath, const TSharedPtr<IContentBrowserHideFolderIfEmptyFilter>& HideFolderIfEmptyFilter, FContentBrowserItemDataTemporaryContext& OutPendingItem);

	/*
	 * Query whether the given item passes the given compiled filter. Should be called after ConvertItemForFilter
	 *
	 * @see CompileFilter.
	 *
	 * @param InItem The item to query.
	 * @param InFilter The compiled filter used to test matching items.
	 *
	 * @return True if the item passes the filter, false otherwise.
	 */
	UE_API virtual bool DoesItemPassFilter(const FContentBrowserItemData& InItem, const FContentBrowserDataCompiledFilter& InFilter);


	/*
	 * Let the compiled filter decide the payload and the type of the item 
	 * Some Compiled filter might change the type/payload of the item. This allow these filter to work properly and should be called before the filtering (see DoesItemPassFilter)
	 * @see CompileFilter
	 * 
	 * @param Item The item that might be converted
	 * @param InFilter The compiled filter used to possibly convert the matching items.
	 * 
	 * @return True if the item was converted by the filter.
	 */
	UE_API virtual bool ConvertItemForFilter(FContentBrowserItemData& Item, const FContentBrowserDataCompiledFilter& InFilter);

	/**
	 * Query the value of the given attribute on the given item.
	 *
	 * @param InItem The item to query.
	 * @param InIncludeMetaData True if we should also include any additional meta-data for the value.
	 * @param InAttributeKey The name of the attribute to query.
	 * @param OutAttributeValue The attribute value to fill with the result.
	 *
	 * @return True if the attribute was found and its value was filled, false otherwise.
	 */
	UE_API virtual bool GetItemAttribute(const FContentBrowserItemData& InItem, const bool InIncludeMetaData, const FName InAttributeKey, FContentBrowserItemDataAttributeValue& OutAttributeValue);

	/**
	 * Query the values of all attributes on the given item.
	 *
	 * @param InItem The item to query.
	 * @param InIncludeMetaData True if we should also include any additional meta-data for the values.
	 * @param OutAttributeValues Map to fill with any attributes available on this item, along with their values.
	 *
	 * @return True attributes were found and their values reported, false otherwise.
	 */
	UE_API virtual bool GetItemAttributes(const FContentBrowserItemData& InItem, const bool InIncludeMetaData, FContentBrowserItemDataAttributeValues& OutAttributeValues);

	/**
	 * Query the physical (on-disk) path of the given item.
	 * @note Not all items or data sources will map to an physical path (eg, some may be purely virtualized views of data that doesn't exist on disk).
	 *
	 * @param InItem The item to query.
	 * @param OutDiskPath The string to fill with the result.
	 *
	 * @return True if the physical path was found and the result filled, false otherwise.
	 */
	UE_API virtual bool GetItemPhysicalPath(const FContentBrowserItemData& InItem, FString& OutDiskPath);

	/**
	 * Query the EAssetAccessSpecifier of the item
	 *
	 * @param InItem The item to query.
	 * @param OutAssetAccessSpecifier The EAssetAccessSpecifier reference that will be filled.
	 *
	 * @return True if OutAssetAccessSpecifier was filled.
	 */
	UE_API virtual bool GetItemAssetAccessSpecifier(const FContentBrowserItemData& InItem, EAssetAccessSpecifier& OutAssetAccessSpecifier);

	/**
	 * Query the item's ability to modify its EAssetAccessSpecifier
	 *
	 * @param InItem The item to query.
	 *
	 * @return True if the item has an EAssetAccessSpecifier that can be modified.
	 */
	UE_API virtual bool CanModifyItemAssetAccessSpecifier(const FContentBrowserItemData& InItem);

	/**
	 * Query whether the given item is considered dirty (ie, has unsaved changes).
	 *
	 * @param InItem The item to query.
	 *
	 * @return True if the item is considered dirty, false otherwise.
	 */
	UE_API virtual bool IsItemDirty(const FContentBrowserItemData& InItem);

	/**
	 * Query whether the given item is can be edited, optionally providing error information if it cannot.
	 *
	 * @param InItem The item to query.
	 * @param OutErrorMessage Optional error message to fill on failure.
	 *
	 * @return True if the item can be edited, false otherwise.
	 */
	UE_API virtual bool CanEditItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg);

	/**
	 * Attempt to open the given item for editing.
	 *
	 * @param InItem The item to edit.
	 *
	 * @return True if the item was opened for editing, false otherwise.
	 */
	UE_API virtual bool EditItem(const FContentBrowserItemData& InItem);

	/**
	 * Attempt to open the given item for editing.
	 * @note The default implementation of this will call EditItem for each item. Override if you can provide a more efficient implementation.
	 *
	 * @param InItems The items to edit.
	 *
	 * @return True if any items were opened for editing, false otherwise.
	 */
	UE_API virtual bool BulkEditItems(TArrayView<const FContentBrowserItemData> InItems);

	/**
     * Query whether the given item is can be viewed (a read-only asset editor), optionally providing error information if it cannot.
     *
     * @param InItem The item to query.
     * @param OutErrorMessage Optional error message to fill on failure.
     *
     * @return True if the item can be viewed in a read-only editor, false otherwise.
     */
    UE_API virtual bool CanViewItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg);

    /**
     * Attempt to open the given item for read-only viewing.
     *
     * @param InItem The item to view.
     *
     * @return True if the item was opened for read-only viewing, false otherwise.
     */
    UE_API virtual bool ViewItem(const FContentBrowserItemData& InItem);

    /**
     * Attempt to open the given items for read-only viewing.
     * @note The default implementation of this will call ViewItem for each item. Override if you can provide a more efficient implementation.
     *
     * @param InItems The items to view.
     *
     * @return True if any items were opened for read-only viewing, false otherwise.
     */
    UE_API virtual bool BulkViewItems(TArrayView<const FContentBrowserItemData> InItems);

	/**
	 * Query whether the given item is can be previewed, optionally providing error information if it cannot.
	 *
	 * @param InItem The item to query.
	 * @param OutErrorMessage Optional error message to fill on failure.
	 *
	 * @return True if the item can be previewed, false otherwise.
	 */
	UE_API virtual bool CanPreviewItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg);

	/**
	 * Attempt to preview the given item.
	 *
	 * @param InItem The item to preview.
	 *
	 * @return True if the item was previewed, false otherwise.
	 */
	UE_API virtual bool PreviewItem(const FContentBrowserItemData& InItem);

	/**
	 * Attempt to preview the given items.
	 * @note The default implementation of this will call PreviewItem for each item. Override if you can provide a more efficient implementation.
	 *
	 * @param InItems The items to preview.
	 *
	 * @return True if any items were previewed, false otherwise.
	 */
	UE_API virtual bool BulkPreviewItems(TArrayView<const FContentBrowserItemData> InItems);

	/**
	 * Query whether the given item is can be duplicated, optionally providing error information if it cannot.
	 *
	 * @param InItem The item to query.
	 * @param OutErrorMessage Optional error message to fill on failure.
	 *
	 * @return True if the item can be duplicated, false otherwise.
	 */
	UE_API virtual bool CanDuplicateItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg);

	/*
	 * Attempt to begin the process of asynchronously duplicating the given item, populating a temporary item that can be finalized or canceled by the user.
	 * @note This duplicates the item at its current path and assigns it a default unique name.
	 *
	 * @param InItem The item to duplicate.
	 * @param OutPendingItem Temporary item context to fill with information about the pending duplicated item.
	 *
	 * @return True if the pending item was created, false otherwise.
	 */
	UE_API virtual bool DuplicateItem(const FContentBrowserItemData& InItem, FContentBrowserItemDataTemporaryContext& OutPendingItem);

	/**
	 * Attempt to synchronously duplicate the given items.
	 * @note This duplicates the items at their current paths and assigns them a default unique name.
	 *
	 * @param InItems The items to duplicate.
	 * @param OutNewItems The new items duplicated from the originals.
	 *
	 * @return True if any items were duplicated, false otherwise.
	 */
	UE_API virtual bool BulkDuplicateItems(TArrayView<const FContentBrowserItemData> InItems, TArray<FContentBrowserItemData>& OutNewItems);

	/**
	 * Query whether the given item is can be saved, optionally providing error information if it cannot.
	 *
	 * @param InItem The item to query.
	 * @param InSaveFlags Flags controlling the rules behind when the item should be saved.
	 * @param OutErrorMessage Optional error message to fill on failure.
	 *
	 * @return True if the item can be saved, false otherwise.
	 */
	UE_API virtual bool CanSaveItem(const FContentBrowserItemData& InItem, const EContentBrowserItemSaveFlags InSaveFlags, FText* OutErrorMsg);

	/**
	 * Attempt to save the given item.
	 *
	 * @param InItem The item to save.
	 * @param InSaveFlags Flags controlling the rules behind when the item should be saved.
	 *
	 * @return True if the item was saved, false otherwise.
	 */
	UE_API virtual bool SaveItem(const FContentBrowserItemData& InItem, const EContentBrowserItemSaveFlags InSaveFlags);

	/**
	 * Attempt to save the given items.
	 * @note The default implementation of this will call SaveItem for each item. Override if you can provide a more efficient implementation.
	 *
	 * @param InItems The items to save.
	 * @param InSaveFlags Flags controlling the rules behind when the item should be saved.
	 *
	 * @return True if any items were saved, false otherwise.
	 */
	UE_API virtual bool BulkSaveItems(TArrayView<const FContentBrowserItemData> InItems, const EContentBrowserItemSaveFlags InSaveFlags);

	/**
	 * Query whether the given item is can be deleted, optionally providing error information if it cannot.
	 *
	 * @param InItem The item to query.
	 * @param OutErrorMessage Optional error message to fill on failure.
	 *
	 * @return True if the item can be deleted, false otherwise.
	 */
	UE_API virtual bool CanDeleteItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg);

	/**
	 * Attempt to delete the given item.
	 *
	 * @param InItem The item to delete.
	 *
	 * @return True if the item was deleted, false otherwise.
	 */
	UE_API virtual bool DeleteItem(const FContentBrowserItemData& InItem);

	/**
	 * Attempt to delete the given items.
	 * @note The default implementation of this will call DeleteItem for each item. Override if you can provide a more efficient implementation.
	 *
	 * @param InItems The items to delete.
	 *
	 * @return True if any items were deleted, false otherwise.
	 */
	UE_API virtual bool BulkDeleteItems(TArrayView<const FContentBrowserItemData> InItems);

	/**
	* Query whether the given item can be privatized, optionally providing error information if it cannot.
	* 
	* @param InItem The item to query.
	* @param OutErrorMessage Optional error message to fill on failure.
	* 
	* @return True if the item was deleted, false otherwise.
	*/
	UE_API virtual bool CanPrivatizeItem(const FContentBrowserItemData& InItem, FText* OutErrorMsg);

	/**
	* Attempt to mark the given item as private (NotExternallyReferenceable).
	* 
	* @param InItem The item to mark private.
	*
	* @return True if the item was marked private, false otherwise
	*/
	UE_API virtual bool PrivatizeItem(const FContentBrowserItemData& InItem, const EAssetAccessSpecifier InAssetAccessSpecifier = EAssetAccessSpecifier::Private);

	/**
	* Attempt to mark the given items as InAssetAccessSpecifier
	* 
	* @param InItems The items to be marked InAssetAccessSpecifier.
	* @param InAssetAccessSpecifier The AssetAccessSpecifier to be marked the assets as.
	* 
	* @return True if any items were marked InAssetAccessSpecifier, false otherwise
	*/
	UE_API virtual bool BulkPrivatizeItems(TArrayView<const FContentBrowserItemData> InItems, const EAssetAccessSpecifier InAssetAccessSpecifier = EAssetAccessSpecifier::Private);

	/**
	 * Query whether the given item is can be renamed, optionally providing error information if it cannot.
	 *
	 * @param InItem The item to query.
	 * @param InNewName Optional name that will be tested for validity alongside the general ability to rename this item.
	 * @param OutErrorMessage Optional error message to fill on failure.
	 *
	 * @return True if the item can be renamed, false otherwise.
	 */
	UE_API virtual bool CanRenameItem(const FContentBrowserItemData& InItem, const FString* InNewName, const IContentBrowserHideFolderIfEmptyFilter* HideFolderIfEmptyFilter, FText* OutErrorMsg);

	/**
	 * Attempt to rename the given item.
	 *
	 * @param InItem The item to rename.
	 * @param InNewName The new name to give the item.
	 * @param OutNewItem The item after it has been renamed.
	 *
	 * @return True if the item was renamed, false otherwise.
	 */
	UE_API virtual bool RenameItem(const FContentBrowserItemData& InItem, const FString& InNewName, FContentBrowserItemData& OutNewItem);

	/**
	 * Query whether the given item is can be copied, optionally providing error information if it cannot.
	 *
	 * @param InItem The item to query.
	 * @param InDestPath The virtual path that the item will be copied to.
	 * @param OutErrorMessage Optional error message to fill on failure.
	 *
	 * @return True if the item can be copied, false otherwise.
	 */
	UE_API virtual bool CanCopyItem(const FContentBrowserItemData& InItem, const FName InDestPath, FText* OutErrorMsg);

	/**
	 * Attempt to copy the given item to the given virtual path.
	 *
	 * @param InItem The item to copy.
	 * @param InDestPath The virtual path that the item will be copied to.
	 *
	 * @return True if the item was copied, false otherwise.
	 */
	UE_API virtual bool CopyItem(const FContentBrowserItemData& InItem, const FName InDestPath);

	/**
	 * Attempt to copy the given items to the given virtual path.
	 * @note The default implementation of this will call CopyItem for each item. Override if you can provide a more efficient implementation.
	 *
	 * @param InItems The items to copy.
	 * @param InDestPath The virtual path that the item will be copied to.
	 *
	 * @return True if any items were copied, false otherwise.
	 */
	UE_API virtual bool BulkCopyItems(TArrayView<const FContentBrowserItemData> InItems, const FName InDestPath);

	/**
	 * Query whether the given item is can be moved, optionally providing error information if it cannot.
	 *
	 * @param InItem The item to query.
	 * @param InDestPath The virtual path that the item will be moved to.
	 * @param OutErrorMessage Optional error message to fill on failure.
	 *
	 * @return True if the item can be moved, false otherwise.
	 */
	UE_API virtual bool CanMoveItem(const FContentBrowserItemData& InItem, const FName InDestPath, FText* OutErrorMsg);

	/**
	 * Attempt to move the given item to the given virtual path.
	 *
	 * @param InItem The item to move.
	 * @param InDestPath The virtual path that the item will be moved to.
	 *
	 * @return True if the item was moved, false otherwise.
	 */
	UE_API virtual bool MoveItem(const FContentBrowserItemData& InItem, const FName InDestPath);

	/**
	 * Attempt to move the given items to the given virtual path.
	 * @note The default implementation of this will call MoveItem for each item. Override if you can provide a more efficient implementation.
	 *
	 * @param InItems The items to move.
	 * @param InDestPath The virtual path that the item will be moved to.
	 *
	 * @return True if any items were moved, false otherwise.
	 */
	UE_API virtual bool BulkMoveItems(TArrayView<const FContentBrowserItemData> InItems, const FName InDestPath);

	/**
	 * Attempt to append any path references for the given item to the given string.
	 * @note Used when copying item references to the clipboard.
	 *
	 * @param InItem The item to query.
	 * @param InOutStr The string to append to (LINE_TERMINATOR delimited).
	 *
	 * @return True if references were appended, false otherwise.
	 */
	UE_API virtual bool AppendItemReference(const FContentBrowserItemData& InItem, FString& InOutStr);

	/**
	 * Attempt to append any object path for the given item to the given string.
	 * @note Used when copying item object path to the clipboard.
	 *
	 * @param InItem The item to query.
	 * @param InOutStr The string to append to (LINE_TERMINATOR delimited).
	 *
	 * @return True if package name were appended, false otherwise.
	 */
	UE_API virtual bool AppendItemObjectPath(const FContentBrowserItemData& InItem, FString& InOutStr);

	/**
	 * Attempt to append any package name for the given item to the given string.
	 * @note Used when copying item package name to the clipboard.
	 *
	 * @param InItem The item to query.
	 * @param InOutStr The string to append to (LINE_TERMINATOR delimited).
	 *
	 * @return True if package name were appended, false otherwise.
	 */
	UE_API virtual bool AppendItemPackageName(const FContentBrowserItemData& InItem, FString& InOutStr);

	/**
	 * Attempt to update the thumbnail associated with the given item.
	 *
	 * @param InItems The items to query.
	 * @param InThumbnail The thumbnail to update.
	 *
	 * @return True if the thumbnail was updated, false otherwise.
	 */
	UE_API virtual bool UpdateThumbnail(const FContentBrowserItemData& InItem, FAssetThumbnail& InThumbnail);

	/**
	 * Called to provide custom drag and drop handling when starting a drag event.
	 * @note If you override this then you are responsible for *all* drag and drop handling (including the default move/copy behavior) via the HandleDrag functions below!
	 *
	 * @param InItems The items being dragged.
	 *
	 * @return A custom drag operation, or null to allow another data source (or the default handler) to deal with it instead.
	 */
	UE_API virtual TSharedPtr<FDragDropOperation> CreateCustomDragOperation(TArrayView<const FContentBrowserItemData> InItems);

	/**
	 * Called to provide custom drag and drop handling when a drag event enters an item, such as performing validation and reporting error information.
	 *
	 * @param InItem The item that the drag entered.
	 * @param InDragDropEvent The drag and drop event.
	 *
	 * @return True if this data source can handle the drag event (even if it won't because it's invalid), or false to allow another data source (or the default handler) to deal with it instead.
	 */
	UE_API virtual bool HandleDragEnterItem(const FContentBrowserItemData& InItem, const FDragDropEvent& InDragDropEvent);

	/**
	 * Called to provide custom drag and drop handling while a drag event is over an item, such as performing validation and reporting error information.
	 *
	 * @param InItem The item that is the current drop target.
	 * @param InDragDropEvent The drag and drop event.
	 *
	 * @return True if this data source can handle the drag event (even if it won't because it's invalid), or false to allow another data source (or the default handler) to deal with it instead.
	 */
	UE_API virtual bool HandleDragOverItem(const FContentBrowserItemData& InItem, const FDragDropEvent& InDragDropEvent);

	/**
	 * Called to provide custom drag and drop handling when a drag event leaves an item, such as clearing any error information set during earlier validation.
	 *
	 * @param InItem The item that the drag left.
	 * @param InDragDropEvent The drag and drop event.
	 *
	 * @return True if this data source can handle the drag event (even if it won't because it's invalid), or false to allow another data source (or the default handler) to deal with it instead.
	 */
	UE_API virtual bool HandleDragLeaveItem(const FContentBrowserItemData& InItem, const FDragDropEvent& InDragDropEvent);

	/**
	 * Called to provide custom drag and drop handling when a drag event is dropped on an item.
	 *
	 * @param InItem The item that was the drop target.
	 * @param InDragDropEvent The drag and drop event.
	 *
	 * @return True if this data source can handle the drag event (even if it didn't because it was invalid), or false to allow another data source (or the default handler) to deal with it instead.
	 */
	UE_API virtual bool HandleDragDropOnItem(const FContentBrowserItemData& InItem, const FDragDropEvent& InDragDropEvent);

	/**
	 * Attempt to retrieve the identifier that should be used when storing a reference to the given item within a collection.
	 *
	 * @param InItem The item to query.
	 * @param OutCollectionId The collection ID to fill.
	 *
	 * @return True if the ID was retrieved, false otherwise.
	 */
	UE_API virtual bool TryGetCollectionId(const FContentBrowserItemData& InItem, FSoftObjectPath& OutCollectionId);

	/**
	 * Attempt to retrieve the package path associated with the given item.
	 * @note This exists to allow the Content Browser to interface with external callbacks that only operate on package paths and should ideally be avoided for new code.
	 * @note Only items which historically represented package paths within the Content Browser should return data from this function (ie, assets and classes).
	 *
	 * @param InItem The item to query.
	 * @param InOutStr The package path to fill.
	 *
	 * @return True if the package path was retrieved, false otherwise.
	 */
	UE_API virtual bool Legacy_TryGetPackagePath(const FContentBrowserItemData& InItem, FName& OutPackagePath);

	/**
	 * Attempt to retrieve the asset data associated with the given item.
	 * @note This exists to allow the Content Browser to interface with external callbacks that only operate on asset data and should ideally be avoided for new code.
	 * @note Only items which historically represented asset data within the Content Browser should return data from this function (ie, assets and classes).
	 *
	 * @param InItem The item to query.
	 * @param InOutStr The asset data to fill.
	 *
	 * @return True if the asset data was retrieved, false otherwise.
	 */
	UE_API virtual bool Legacy_TryGetAssetData(const FContentBrowserItemData& InItem, FAssetData& OutAssetData);

	/**
	 * Attempt to convert the given package path to a virtual path associated with this data source.
	 * @note This exists to allow the Content Browser to interface with public APIs that only operate on package paths and should ideally be avoided for new code.
	 * @note This function only adjusts the path to something that could represent a virtualized item within this data source, but it doesn't guarantee that an item actually exists at that path.
	 *
	 * @param InPackagePath The package path to query.
	 * @param OutPath The virtualized path to fill.
	 *
	 * @return True if the package path was mapped, false otherwise.
	 */
	UE_API virtual bool Legacy_TryConvertPackagePathToVirtualPath(const FName InPackagePath, FName& OutPath);

	/**
	 * Attempt to convert the given asset data to a virtual path associated with this data source.
	 * @note This exists to allow the Content Browser to interface with public APIs that only operate on asset data and should ideally be avoided for new code.
	 * @note This function only adjusts the path to something that could represent a virtualized item within this data source, but it doesn't guarantee that an item actually exists at that path.
	 *
	 * @param InAssetData The asset data to query.
	 * @param InUseFolderPaths True if this conversion is for the paths view (so should use the parent folder of an asset), or false if it is for the asset view.
	 * @param OutPath The virtualized path to fill.
	 *
	 * @return True if the asset data was mapped, false otherwise.
	 */
	UE_API virtual bool Legacy_TryConvertAssetDataToVirtualPath(const FAssetData& InAssetData, const bool InUseFolderPaths, FName& OutPath);

	/**
	 * Sets a flag to force rebuild of virtual path tree with next call to RefreshVirtualPathTreeIfNeeded()
	 */
	UE_API void SetVirtualPathTreeNeedsRebuild();

	/**
	 * Call after a change that could affect rules of virtual path generation.
	 */
	UE_API void RefreshVirtualPathTreeIfNeeded();

	/**
	 * Attempt to convert the given virtual path
	 * @note Does test if virtual portion of path exists
	 * @note Does not test if internal portion of path exists
	 *
	 * @return None if virtual path prefix of InPath does not exist, Virtual if path exists and is fully virtual (stops before it reaches internal root), Internal if virtual path part of prefix exists and there is text after the virtual prefix
	 */
	UE_API EContentBrowserPathType TryConvertVirtualPath(const FStringView InPath, FStringBuilderBase& OutPath) const;
	UE_API EContentBrowserPathType TryConvertVirtualPath(const FStringView InPath, FString& OutPath) const;
	UE_API EContentBrowserPathType TryConvertVirtualPath(const FStringView InPath, FName& OutPath) const;
	UE_API EContentBrowserPathType TryConvertVirtualPath(const FName InPath, FName& OutPath) const;

	/**
	 * Rebuilds the tree of virtual paths that ends with internal roots
	 */
	UE_API virtual void BuildRootPathVirtualTree();

	const FContentBrowserVirtualPathTree& GetRootPathVirtualTree() const { return RootPathVirtualTree; }

	/**
	 * Creates item data for a fully virtual folder.
	 */
	UE_API FContentBrowserItemData CreateVirtualFolderItem(const FName InFolderPath);

	/**
	 * Convert a virtualized path to its internal form, based on the mount root set on this data source.
	 * @note The default implementation expects to produce a package path like result, eg) "/Folder/Folder/File".
	 *
	 * @param InPath The virtualized path to query.
	 * @param OutInternalPath The internal path to fill.
	 *
	 * @return True if the virtual path was mapped, false otherwise.
	 */
	UE_API virtual bool TryConvertVirtualPathToInternal(const FName InPath, FName& OutInternalPath);

	/**
	 * Convert an internal path to its virtualized form, based on the mount root set on this data source.
	 * @note The default implementation expects to consume a package path like input, eg) "/Folder/Folder/File".
	 *
	 * @param InInternalPath The internal path to query.
	 * @param OutPath The virtual path to fill.
	 *
	 * @return True if the internal path was mapped, false otherwise.
	 */
	UE_API virtual bool TryConvertInternalPathToVirtual(const FName InInternalPath, FName& OutPath);

	/**
	 * Tell the data source to remove any cached data for the filter compilation that might not be needed any more.
	 */
	UE_API virtual void RemoveUnusedCachedFilterData(const FContentBrowserDataFilterCacheIDOwner& IDOwner, TArrayView<const FName> InVirtualPathsInUse, const FContentBrowserDataFilter& DataFilter);

	/**
	 * Tell the data source to remove the cached data for the filter compilation for this specific owner. 
	 */
	UE_API virtual void ClearCachedFilterData(const FContentBrowserDataFilterCacheIDOwner& IDOwner);

protected:

	/**
	 * Queue an incremental item data update, for data sources that can provide delta-updates.
	 * These updates are flushed out at the end of the next call to Tick on the Content Browser Data Subsystem.
	 *
	 * @param InUpdate The update describing how the item has changed.
	 */
	UE_API void QueueItemDataUpdate(FContentBrowserItemDataUpdate&& InUpdate);

	/**
	 * Notify a wholesale item data update, for data sources that can't provide delta-updates.
	 */
	 UE_DEPRECATED(5.5, "No data source should be able request a full data refresh. On large projects this can quickly destroy the editor performance. To avoid that all data sources should support queueing the required incremental data updates for their items. Look for uses of the function QueueItemDataUpdate for examples.")
	UE_API void NotifyItemDataRefreshed();

	/**
	 * Adds internal root path to virtual path tree
	 */
	UE_API void RootPathAdded(const FStringView InInternalPath);

	/**
	 * Removes internal root path from virtual path tree
	 */
	UE_API void RootPathRemoved(const FStringView InInternalPath);


	/**
	 * Tree of virtual paths that ends with internal roots. Used for enumeration and conversion of paths.
	 */
	FContentBrowserVirtualPathTree RootPathVirtualTree;

private:
	/**
	 * True if this data source is currently initialized.
	 */
	bool bIsInitialized = false;

	/**
	 * True if this data source's virtual path tree needs rebuilding.
	 */
	bool bVirtualPathTreeNeedsRebuild = true;

	struct FVirtualPathTreeRulesCachedState
	{
		bool bShowAllFolder = false;
		bool bOrganizeFolders = false;
	};

	/**
	 * Cached state of rules used to detect when virtual path tree needs rebuilding
	 */
	FVirtualPathTreeRulesCachedState VirtualPathTreeRulesCachedState;

	/**
	 * The data sink that can be used to communicate with the Content Browser Data Subsystem.
	 * This is set to a valid instance when the data source is activated, and set to null when it is deactivated.
	 */
	IContentBrowserItemDataSink* DataSink = nullptr;
};

#undef UE_API
