// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SortedMap.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/Ticker.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDataFilter.h"
#include "ContentBrowserItem.h"
#include "ContentBrowserItemData.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "EditorSubsystem.h"
#include "GetOrEnumerateSink.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Misc/NamePermissionList.h"
#include "Misc/StringBuilder.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ContentBrowserDataSubsystem.generated.h"

#define UE_API CONTENTBROWSERDATA_API

class FSubsystemCollectionBase;
class FText;
class IPlugin;
class UContentBrowserDataSource;
class UObject;
struct FAssetData;
struct FContentBrowserItemPath;
struct FFrame;
template <typename FuncType> class TFunctionRef;

UENUM(BlueprintType)
enum class EContentBrowserPathType : uint8
{
	/** No path type set */
	None,
	/** Internal path compatible with asset registry and engine calls (eg,. "/PluginA/MyFile") */
	Internal,
	/** Virtual path for enumerating Content Browser data (eg, "/All/Plugins/PluginA/MyFile") */
	Virtual
};

enum class EContentBrowserIsFolderVisibleFlags : uint8
{
	None = 0,

	/**
	 * Hide folders that recursively contain no file items.
	 */
	HideEmptyFolders UE_DEPRECATED(5.5, "Empty folder filtering is now dependent on which types of contents are visible, use the FContentBrowserFolderContentsFilter argument to IsFolderVisible") = 1 << 0,

	/**
	 * Default visibility flags.
	 */
	Default = None,
};
ENUM_CLASS_FLAGS(EContentBrowserIsFolderVisibleFlags);

/** Called for incremental item data updates from data sources that can provide delta-updates */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnContentBrowserItemDataUpdated, TArrayView<const FContentBrowserItemDataUpdate>);

/** Called for wholesale item data updates from data sources that can't provide delta-updates, or when the set of active data sources is modified */
DECLARE_MULTICAST_DELEGATE(FOnContentBrowserItemDataRefreshed);

/** Called when all active data sources have completed their initial content discovery scan. May be called multiple times if new data sources are registered after the current set of active data sources have completed their initial scan */
DECLARE_MULTICAST_DELEGATE(FOnContentBrowserItemDataDiscoveryComplete);

/** Called when generating a virtual path, allows customization of how a virtual path is generated. */
DECLARE_DELEGATE_TwoParams(FContentBrowserGenerateVirtualPathDelegate, const FStringView, FStringBuilderBase&);

/** Called to check whether the plugin VersePath should be its path in the content browser, if it has one */
DECLARE_DELEGATE_RetVal_OneParam(bool, FContentBrowserUsePluginVersePathDelegate, const TSharedRef<IPlugin>&);

/** Called to create a IContentBrowserHideFolderIfEmptyFilter filter */
DECLARE_DELEGATE_RetVal(TSharedPtr<IContentBrowserHideFolderIfEmptyFilter>, FContentBrowserCreateHideFolderIfEmptyFilter);

/** Internal - Filter data used to inject dummy items for the path down to the mount root of each data source */
USTRUCT()
struct FContentBrowserCompiledSubsystemFilter
{
	GENERATED_BODY()
	
public:
	TArray<FName> MountRootsToEnumerate;
};

/** Internal - Filter data used to inject dummy items */
USTRUCT()
struct FContentBrowserCompiledVirtualFolderFilter
{
	GENERATED_BODY()

public:
	TMap<FName, FContentBrowserItemData> CachedSubPaths;
};

/**
 * Subsystem that provides access to Content Browser data.
 * This type deals with the composition of multiple data sources, which provide information about the folders and files available in the Content Browser.
 */
UCLASS(MinimalAPI, config=Editor)
class UContentBrowserDataSubsystem : public UEditorSubsystem, public IContentBrowserItemDataSink
{
	GENERATED_BODY()

public:
	friend class FScopedSuppressContentBrowserDataTick;

	UE_API UContentBrowserDataSubsystem();
	//~ UEditorSubsystem interface
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	UE_API virtual void ConvertInternalPathToVirtual(const FStringView InPath, FStringBuilderBase& OutPath) override;

	/**
	 * Attempt to activate the named data source.
	 * @return True if the data source was available and not already active, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="ContentBrowser")
	UE_API bool ActivateDataSource(const FName Name);

	/**
	 * Attempt to deactivate the named data source.
	 * @return True if the data source was available and active, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="ContentBrowser")
	UE_API bool DeactivateDataSource(const FName Name);

	/**
	 * Activate all available data sources.
	 */
	UFUNCTION(BlueprintCallable, Category="ContentBrowser")
	UE_API void ActivateAllDataSources();

	/**
	 * Deactivate all active data sources.
	 */
	UFUNCTION(BlueprintCallable, Category="ContentBrowser")
	UE_API void DeactivateAllDataSources();

	/**
	 * Get the list of current available data sources.
	 */
	UFUNCTION(BlueprintCallable, Category="ContentBrowser")
	UE_API TArray<FName> GetAvailableDataSources() const;

	/**
	 * Get the list of current active data sources.
	 */
	UFUNCTION(BlueprintCallable, Category="ContentBrowser")
	UE_API TArray<FName> GetActiveDataSources() const;

	/**
	 * Delegate called for incremental item data updates from data sources that can provide delta-updates.
	 */
	UE_API FOnContentBrowserItemDataUpdated& OnItemDataUpdated();

	/**
	 * Delegate called for wholesale item data updates from data sources that can't provide delta-updates, or when the set of active data sources is modified.
	 */
	UE_API FOnContentBrowserItemDataRefreshed& OnItemDataRefreshed();

	/**
	 * Delegate called when all active data sources have completed their initial content discovery scan.
	 * @note May be called multiple times if new data sources are registered after the current set of active data sources have completed their initial scan.
	 */
	UE_API FOnContentBrowserItemDataDiscoveryComplete& OnItemDataDiscoveryComplete();

	/**
	 * Take a raw data filter and convert it into a compiled version that could be re-used for multiple queries using the same data (typically this is only useful for post-filtering multiple items).
	 * @note The compiled filter is only valid until the data source changes, so only keep it for a short time (typically within a function call, or 1-frame).
	 */
	UE_API void CompileFilter(const FName InPath, const FContentBrowserDataFilter& InFilter, FContentBrowserDataCompiledFilter& OutCompiledFilter) const;

	/**
	 * Enumerate the items (folders and/or files) that match a previously compiled filter.
	 */
	UE_API void EnumerateItemsMatchingFilter(
		const FContentBrowserDataCompiledFilter& InFilter, TFunctionRef<bool(FContentBrowserItem&&)> InCallback) const;
	UE_API void EnumerateItemsMatchingFilter(const FContentBrowserDataCompiledFilter& InFilter,
		TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) const;
	UE_API void EnumerateItemsMatchingFilter(
		const FContentBrowserDataCompiledFilter& InFilter, const TGetOrEnumerateSink<FContentBrowserItemData>& InSink) const;

	/**
	 * Enumerate the items (folders and/or files) that exist under the given virtual path.
	 */
	UE_API void EnumerateItemsUnderPath(const FName InPath,
		const FContentBrowserDataFilter& InFilter,
		TFunctionRef<bool(FContentBrowserItem&&)> InCallback) const;
	UE_API void EnumerateItemsUnderPath(const FName InPath,
		const FContentBrowserDataFilter& InFilter,
		TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) const;
	// Version which allows passing an array or a callback, allows sources to presize the array for large queries
	UE_API void EnumerateItemsUnderPath(const FName InPath,
		const FContentBrowserDataFilter& InFilter,
		const TGetOrEnumerateSink<FContentBrowserItemData>& InSink) const;

	/**
	 * Get the items (folders and/or files) that exist under the given virtual path.
	 */
	UFUNCTION(BlueprintCallable, Category="ContentBrowser")
	UE_API TArray<FContentBrowserItem> GetItemsUnderPath(const FName InPath, const FContentBrowserDataFilter& InFilter) const;

	/**
	 * Enumerate the items (folders and/or files) that exist at the given virtual path.
	 * @note Multiple items may have the same virtual path if they are different types, or come from different data sources.
	 */
	UE_API void EnumerateItemsAtPath(const FName InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItem&&)> InCallback) const;
	UE_API void EnumerateItemsAtPath(const FName InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) const;

	/**
	 * Enumerate the items (files) that exist at the given paths.
	 * @note Multiple items may have the same virtual path if they are different types, or come from different data sources.
	 *
	 * @param InItemPaths The paths to enumerate
	 * @param InItemTypeFilter The types of items we want to find.
	 * @param InCallback The function to invoke for each matching item (return true to continue enumeration).
	 */
	UE_API bool EnumerateItemsAtPaths(const TArrayView<struct FContentBrowserItemPath> InItemPaths, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) const;

	/**
	 * Enumerate the items (folders and/or files) that exist at the given user provided path.
	 * The user provided path can be any path format any of the data sources support other than a virtual path. The data sources will deduce the path type.
	 * @note Multiple items may have the same virtual path if they are different types, or come from different data sources.
	 */
	UE_API void EnumerateItemsAtUserProvidedPath(const FStringView InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItem&&)> InCallback) const;
	UE_API void EnumerateItemsAtUserProvidedPath(const FStringView InPath, const EContentBrowserItemTypeFilter InItemTypeFilter, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) const;

	/**
	 * Enumerate the items (files) that exist for the given objects.
	 * @note Multiple items may have the same virtual path if they are different types, or come from different data sources.
	 *
	 * @param InObjects The objects to enumerate
	 * @param InCallback The function to invoke for each matching item (return true to continue enumeration).
	 */
	UE_API bool EnumerateItemsForObjects(const TArrayView<UObject*> InObjects, TFunctionRef<bool(FContentBrowserItemData&&)> InCallback) const;

	/**
	 * Get the items (folders and/or files) that exist at the given virtual path.
	 * @note Multiple items may have the same virtual path if they are different types, or come from different data sources.
	 */
	UFUNCTION(BlueprintCallable, Category="ContentBrowser")
	UE_API TArray<FContentBrowserItem> GetItemsAtPath(const FName InPath, const EContentBrowserItemTypeFilter InItemTypeFilter) const;

	/**
	 * Get the first item (folder and/or file) that exists at the given virtual path.
	 */
	UFUNCTION(BlueprintCallable, Category="ContentBrowser")
	UE_API FContentBrowserItem GetItemAtPath(const FName InPath, const EContentBrowserItemTypeFilter InItemTypeFilter) const;

	/**
	 * Get the items (folders and/or files) that exist at the given user provided path.
	 * The user provided path can be any path format any of the data sources support other than a virtual path. The data sources will deduce the path type.
	 * @note Multiple items may have the same virtual path if they are different types, or come from different data sources.
	 */
	UE_API TArray<FContentBrowserItem> GetItemsAtUserProvidedPath(const FStringView InPath, const EContentBrowserItemTypeFilter InItemTypeFilter) const;

	/**
	 * Get a list of other paths that the data source may be using to represent a specific path
	 *
	 * @param The internal path (or object path) of an asset to get aliases for
	 * @return All alternative paths that represent the input path (not including the input path itself)
	 */
	UE_API TArray<FContentBrowserItemPath> GetAliasesForPath(const FSoftObjectPath& InInternalPath) const;
	UE_API TArray<FContentBrowserItemPath> GetAliasesForPath(const FContentBrowserItemPath InPath) const;
	UE_API TArray<FContentBrowserItemPath> GetAliasesForPath(const FName InInternalPath) const;

	/**
	 * Query whether any data sources are currently discovering content, and retrieve optional status messages that can be shown in the UI.
	 */
	UE_API bool IsDiscoveringItems(TArray<FText>* OutStatus = nullptr) const;

	/**
	 * If possible, attempt to prioritize content discovery for the given virtual path.
	 */
	UE_API bool PrioritizeSearchPath(const FName InPath);

	/**
	 * Query whether the given virtual folder should be visible in the UI.
	 * @param Path Virtual path of folder e.g. /All/GameData/Stuff
	 * @param Flags Currently unused
	 */
	UE_API bool IsFolderVisible(const FName Path, const EContentBrowserIsFolderVisibleFlags Flags = EContentBrowserIsFolderVisibleFlags::Default) const;

	/**
	 * Query whether the given virtual folder should be visible in the UI.
	 * @param Path Virtual path of folder e.g. /All/GameData/Stuff
	 * @param Flags Currently unused
	 * @param ContentsFilter Filter for limiting visibility to only folders with a certain type of contents based on desired view filtering.
	 */
	UE_API bool IsFolderVisible(const FName Path, const EContentBrowserIsFolderVisibleFlags Flags, const FContentBrowserFolderContentsFilter& ContentsFilter) const;

	/**
	 * Query whether the given virtual folder should be visible if the UI is asking to hide empty content folders.
	 */
	UE_DEPRECATED(5.3, "IsFolderVisibleIfHidingEmpty is deprecated. Use IsFolderVisible instead and add EContentBrowserIsFolderVisibleFlags::HideEmptyFolders to the flags.")
	UE_API bool IsFolderVisibleIfHidingEmpty(const FName InPath) const;

	/*
	 * Query whether a folder can be created at the given virtual path, optionally providing error information if it cannot.
	 *
	 * @param InPath The virtual path of the folder that is being queried.
	 * @param OutErrorMessage Optional error message to fill on failure.
	 *
	 * @return True if the folder can be created, false otherwise.
	 */
	UE_API bool CanCreateFolder(const FName InPath, FText* OutErrorMsg) const;

	/*
	 * Attempt to begin the process of asynchronously creating a folder at the given virtual path, returning a temporary item that can be finalized or canceled by the user.
	 *
	 * @param InPath The initial virtual path of the folder that is being created.
	 *
	 * @return The pending folder item to create (test for validity).
	 */
	UE_API FContentBrowserItemTemporaryContext CreateFolder(const FName InPath) const;

	/*
	 * Attempt to begin the process of asynchronously creating a folder at the given virtual path, returning a temporary item that can be finalized or canceled by the user.
	 *
	 * @param InPath The initial virtual path of the folder that is being created.
	 * @param HideFolderIfEmptyFilter Current empty folder filter used to permit new folder operations on hidden folders.
	 *
	 * @return The pending folder item to create (test for validity).
	 */
	UE_API FContentBrowserItemTemporaryContext CreateFolder(const FName InPath, const TSharedPtr<IContentBrowserHideFolderIfEmptyFilter>& HideFolderIfEmptyFilter) const;

	/**
	 * Attempt to convert the given package path to virtual paths associated with the active data sources (callback will be called for each successful conversion).
	 * @note This exists to allow the Content Browser to interface with public APIs that only operate on package paths and should ideally be avoided for new code.
	 * @note This function only adjusts the path to something that could represent a virtualized item within this data source, but it doesn't guarantee that an item actually exists at that path.
	 */
	UE_API void Legacy_TryConvertPackagePathToVirtualPaths(const FName InPackagePath, TFunctionRef<bool(FName)> InCallback);

	/**
	 * Attempt to convert the given asset data to a virtual paths associated with the active data sources (callback will be called for each successful conversion).
	 * @note This exists to allow the Content Browser to interface with public APIs that only operate on asset data and should ideally be avoided for new code.
	 * @note This function only adjusts the path to something that could represent a virtualized item within this data source, but it doesn't guarantee that an item actually exists at that path.
	 */
	UE_API void Legacy_TryConvertAssetDataToVirtualPaths(const FAssetData& InAssetData, const bool InUseFolderPaths, TFunctionRef<bool(FName)> InCallback);

	/**
	 * Rebuild the virtual path tree if rules have changed.
	 */
	UE_API void RefreshVirtualPathTreeIfNeeded();

	/**
	 * Call when rules of virtual path generation have changed beyond content browser settings.
	 */
	UE_API void SetVirtualPathTreeNeedsRebuild();

	/**
	 * Converts an internal path to a virtual path based on current rules
	 */
	UE_API void ConvertInternalPathToVirtual(const FStringView InPath, FName& OutPath);
	UE_API void ConvertInternalPathToVirtual(FName InPath, FName& OutPath);
	UE_API FName ConvertInternalPathToVirtual(FName InPath);
	UE_API TArray<FString> ConvertInternalPathsToVirtual(const TArray<FString>& InPaths);

	/**
	 * Converts a virtual path to a display path, where each path segment is replaced with the display name of its corresponding FContentBrowserItem.
	 * This path will display similar to the Content Browser's navigation bar.
	 */
	UE_API FText ConvertVirtualPathToDisplay(const FStringView InVirtualPath, const EContentBrowserItemTypeFilter InLeafType) const;
	UE_API FText ConvertVirtualPathToDisplay(FName InVirtualPath, const EContentBrowserItemTypeFilter InLeafType) const;
	UE_API FText ConvertVirtualPathToDisplay(const FContentBrowserItem& InItem) const;

	/**
	 * Converts virtual path back into an internal or invariant path
	 */
	UE_API EContentBrowserPathType TryConvertVirtualPath(const FStringView InPath, FStringBuilderBase& OutPath) const;
	UE_API EContentBrowserPathType TryConvertVirtualPath(const FStringView InPath, FString& OutPath) const;
	UE_API EContentBrowserPathType TryConvertVirtualPath(const FStringView InPath, FName& OutPath) const;
	UE_API EContentBrowserPathType TryConvertVirtualPath(const FName InPath, FName& OutPath) const;

	/**
	 * Returns array of paths converted to internal.
	 */
	UE_API TArray<FString> TryConvertVirtualPathsToInternal(const TArray<FString>& InVirtualPaths) const;

	/**
	 * Customize list of folders that appear first in content browser based on internal or invariant paths
	 */
	UE_API void SetPathViewSpecialSortFolders(const TArray<FName>& InSpecialSortFolders);

	/**
	 * Returns reference to list of paths that appear first in content browser based on internal or invariant paths
	 */
	UE_API const TArray<FName>& GetPathViewSpecialSortFolders() const;

	/**
	 * Returns reference to default list of paths that appear first in content browser based on internal or invariant paths
	 */
	UE_API const TArray<FName>& GetDefaultPathViewSpecialSortFolders() const;

	/**
	 * Set delegate used to generate a virtual path.
	 */
	UE_API void SetGenerateVirtualPathPrefixDelegate(const FContentBrowserGenerateVirtualPathDelegate& InDelegate);

	/**
	 * Delegate called to generate a virtual path. Can be set to override default behavior.
	 */
	UE_API FContentBrowserGenerateVirtualPathDelegate& OnGenerateVirtualPathPrefix();

	/**
	 * Return whether the plugin VersePath should be its path in the content browser, if it has one
	 */
	UE_API bool UsePluginVersePath(const TSharedRef<IPlugin>& Plugin);

	/**
	 * Set delegate to check whether the plugin VersePath should be its path in the content browser, if it has one
	 */
	UE_API void SetUsePluginVersePathDelegate(FContentBrowserUsePluginVersePathDelegate InDelegate);

	/**
	 * Delegate to check whether the plugin VersePath should be its path in the content browser, if it has one
	 */
	UE_API FContentBrowserUsePluginVersePathDelegate& GetUsePluginVersePathDelegate();

	/**
	 * Creates the default IContentBrowserHideFolderIfEmptyFilter, and combines it with the results of any additional registered FContentBrowserCreateHideFolderIfEmptyFilter delegates.
	 * @note The filter is only valid until any dependent data changes, so only keep it for a short time (typically within a function call tree).
	 */
	UE_API TSharedPtr<IContentBrowserHideFolderIfEmptyFilter> CreateHideFolderIfEmptyFilter() const;

	/**
	* Registers/deregisters a delegate used to create a filter used to filter out empty folders. If multiple delegates are registered they will get or'd togeather.
	* Intended for content folders that always exist but do not generally contain assets, such as /Game/Collections
	*/
	UE_API FDelegateHandle RegisterCreateHideFolderIfEmptyFilter(FContentBrowserCreateHideFolderIfEmptyFilter Delegate);
	UE_API void UnregisterCreateHideFolderIfEmptyFilter(FDelegateHandle DelegateHandle);

	/**
	 * Prefix to use when generating virtual paths and "Show All Folder" option is enabled
	 */
	UE_API const FString& GetAllFolderPrefix() const;

	/**
	 * Permission list that controls whether content in a given folder path can be edited.
	 * Note: This does not control if the folder path is writable or if content can be deleted.
	 */
	UE_API TSharedRef<FPathPermissionList>& GetEditableFolderPermissionList();


	/**
	 * Provide an partial access to private api for the filter cache ID owners
	 * See FContentBrowserDataFilterCacheIDOwner declaration for how to use the caching for filter compilation
	 */
	struct FContentBrowserFilterCacheApi
	{
	private:
		friend FContentBrowserDataFilterCacheIDOwner;

		static void InitializeCacheIDOwner(UContentBrowserDataSubsystem& Subsystem, FContentBrowserDataFilterCacheIDOwner& IDOwner);
		static void RemoveUnusedCachedData(const UContentBrowserDataSubsystem& Subsystem, const FContentBrowserDataFilterCacheIDOwner& IDOwner, TArrayView<const FName> InVirtualPathsInUse, const FContentBrowserDataFilter& DataFilter);
		static void ClearCachedData(const UContentBrowserDataSubsystem& Subsystem, const FContentBrowserDataFilterCacheIDOwner& IDOwner);
	};

private:

	UE_API void InitializeCacheIDOwner(FContentBrowserDataFilterCacheIDOwner& IDOwner);

	UE_API void RemoveUnusedCachedFilterData(const FContentBrowserDataFilterCacheIDOwner& IDOwner, TArrayView<const FName> InVirtualPathsInUse, const FContentBrowserDataFilter& DataFilter) const;

	UE_API void ClearCachedFilterData(const FContentBrowserDataFilterCacheIDOwner& IDOwner) const;

	using FNameToDataSourceMap = TSortedMap<FName, UContentBrowserDataSource*, FDefaultAllocator, FNameFastLess>;

	/**
	 * Called to handle a data source modular feature being registered.
	 * @note Will activate the data source if it is in the EnabledDataSources array.
	 */
	UE_API void HandleDataSourceRegistered(const FName& Type, class IModularFeature* Feature);
	
	/**
	 * Called to handle a data source modular feature being unregistered.
	 * @note Will deactivate the data source if it is in the ActiveDataSources map.
	 */
	UE_API void HandleDataSourceUnregistered(const FName& Type, class IModularFeature* Feature);

	/**
	 * Tick this subsystem.
	 * @note Called once every 0.1 seconds.
	 */
	UE_API void Tick(const float InDeltaTime);

	/**
	 * Returns true if item data modifications are being processed.
	 */
	UE_API bool AllowModifiedItemDataUpdates() const;

	/**
	 * Called when Play in Editor begins.
	 */
	UE_API void OnBeginPIE(const bool bIsSimulating);

	/**
	 * Called when Play in Editor stops.
	 */
	UE_API void OnEndPIE(const bool bIsSimulating);

	/**
	 * Called when Content added to delay content browser tick for a frame
	 * Prevents content browser from slowing down initialization by ticking as content is loaded
	 */
	UE_API void OnContentPathMounted(const FString& AssetPath, const FString& ContentPath);

	//~ IContentBrowserItemDataSink interface
	UE_API virtual void QueueItemDataUpdate(FContentBrowserItemDataUpdate&& InUpdate) override;

	UE_DEPRECATED(5.5, "NotifyItemDataRefreshed is deprecated, for editor performance reasons no external systems should be able to request a full refresh.")
	UE_API virtual void NotifyItemDataRefreshed() override;

	/**
	 * Handle for the Tick callback.
	 */
	FTSTicker::FDelegateHandle TickHandle;

	/**
	 * Map of data sources that are currently active.
	 */
	FNameToDataSourceMap ActiveDataSources;

	/**
	 * Map of data sources that are currently available.
	 */
	FNameToDataSourceMap AvailableDataSources;

	/**
	 * Set of data sources that are currently running content discovery.
	 * ItemDataDiscoveryCompleteDelegate will be called each time this set becomes empty.
	 */
	TSet<FName> ActiveDataSourcesDiscoveringContent;

	/**
	 * Array of data source names that should be activated when available.
	 */
	UPROPERTY(config)
	TArray<FName> EnabledDataSources;

	/**
	 * Queue of incremental item data updates.
	 * These will be passed to ItemDataUpdatedDelegate on the end of Tick.
	 */
	TArray<FContentBrowserItemDataUpdate> PendingUpdates;

	/**
	 * Set of item data updates that are delayed to preserve the editor performance.
	 * These will be passed to the PendingUpdates when we exit a pie session.
	 */
	TMap<FContentBrowserItemKey, FContentBrowserItemDataUpdate> DelayedPendingUpdates;

	/**
	 * True if an item data refresh notification is pending.
	 */
	bool bPendingItemDataRefreshedNotification = false;

	/**
	 * True if Play in Editor is active.
	 */
	bool bIsPIEActive = false;

	/**
	 * True if content was just mounted this frame
	 */
	bool bContentMountedThisFrame = false;

	/**
	 * >0 if Tick events have currently been suppressed.
	 * @see FScopedSuppressContentBrowserDataTick
	 */
	int32 TickSuppressionCount = 0;

	/**
	 * Delegate called for incremental item data updates from data sources that can provide delta-updates.
	 */
	FOnContentBrowserItemDataUpdated ItemDataUpdatedDelegate;

	/**
	 * Delegate called for wholesale item data updates from data sources that can't provide delta-updates, or when the set of active data sources is modified.
	 */
	FOnContentBrowserItemDataRefreshed ItemDataRefreshedDelegate;

	/**
	 * Delegate called when all active data sources have completed their initial content discovery scan.
	 * @note May be called multiple times if new data sources are registered after the current set of active data sources have completed their initial scan.
	 */
	FOnContentBrowserItemDataDiscoveryComplete ItemDataDiscoveryCompleteDelegate;

	/**
	 * Generates an optional virtual path prefix for a given internal path
	 */
	FContentBrowserGenerateVirtualPathDelegate GenerateVirtualPathPrefixDelegate;

	/**
	 * Delegate to check whether the plugin VersePath should be its path in the content browser, if it has one
	 */
	FContentBrowserUsePluginVersePathDelegate UsePluginVersePathDelegate;

	/**
	 * Default filter used to check whether an empty folder should always be filtered out in the Content Browser
	 */
	TSharedPtr<IContentBrowserHideFolderIfEmptyFilter> DefaultHideFolderIfEmptyFilter;

	/**
	 * Delegates to create filters that check whether an empty folder should always be filtered out in the Content Browser
	 */
	TArray<FContentBrowserCreateHideFolderIfEmptyFilter> CreateHideFolderIfEmptyFilterDelegates;

	/**
	 * Optional array of invariant paths to use when sorting
	 */
	TArray<FName> PathViewSpecialSortFolders;

	/**
	 * Default array of invariant paths to use when sorting
	 */
	TArray<FName> DefaultPathViewSpecialSortFolders;

	/**
	 * Prefix to use when generating virtual paths and "Show All Folder" option is enabled
	 */
	FString AllFolderPrefix;

	/** Permission list of folder paths we can edit */
	TSharedRef<FPathPermissionList> EditableFolderPermissionList;

	int64 LastCacheIDForFilter = INDEX_NONE;
};

/**
 * Helper to suppress Tick events during critical times, when the underlying data should not be updated.
 */
class FScopedSuppressContentBrowserDataTick
{
public:
	explicit FScopedSuppressContentBrowserDataTick(UContentBrowserDataSubsystem* InContentBrowserData)
		: ContentBrowserData(InContentBrowserData)
	{
		check(ContentBrowserData);
		++ContentBrowserData->TickSuppressionCount;
	}

	~FScopedSuppressContentBrowserDataTick()
	{
		checkf(ContentBrowserData->TickSuppressionCount > 0, TEXT("TickSuppressionCount underflow!"));
		--ContentBrowserData->TickSuppressionCount;
	}

	FScopedSuppressContentBrowserDataTick(const FScopedSuppressContentBrowserDataTick&) = delete;
	FScopedSuppressContentBrowserDataTick& operator=(const FScopedSuppressContentBrowserDataTick&) = delete;

private:
	UContentBrowserDataSubsystem* ContentBrowserData = nullptr;
};

#undef UE_API
