// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDataFilter.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"

class UObject;

struct FAssetData;

namespace ContentBrowserDataUtils
{
	/** Returns number of folders in forward slashed path (Eg, 1 for "/Path", 2 for "/Path/Name") */
	CONTENTBROWSERDATA_API int32 CalculateFolderDepthOfPath(const FStringView InPath);

	/** Returns true if folder has a depth of 1 */
	CONTENTBROWSERDATA_API bool IsTopLevelFolder(const FStringView InFolderPath);

	/** Returns true if folder has a depth of 1 */
	CONTENTBROWSERDATA_API bool IsTopLevelFolder(const FName InFolderPath);

	/** Return the test depth after which it is not needed to test the attribute filter if the parent folder was tested */
	CONTENTBROWSERDATA_API int32 GetMaxFolderDepthRequiredForAttributeFilter();

	/**
	 * Tests internal path against attribute filter
	 * 
	 * @param InPath Invariant path to test
	 * @param InAlreadyCheckedDepth Number of folders deep that have already been tested to avoid re-testing during recursion. Pass 0 if portion of path not already tested.
	 * @param InItemAttributeFilter Filter to test against
	 * 
	 * @return True if passes filter
	 */
	CONTENTBROWSERDATA_API bool PathPassesAttributeFilter(const FStringView InPath, const int32 InAlreadyCheckedDepth, const EContentBrowserItemAttributeFilter InAttributeFilter);

	/**
	 * Get display name override if there is one for InFolderPath
	 * 
	 * @param InFolderPath Internal path to get display name override for
	 * @param InFolderItemName Short name of InFolderPath (rightmost folder name)
	 * @param bIsClassesFolder True if this folder is a classes folder
	 * @param bIsCookedPath True if this folder only contains cooked content (recursively), or false if it contains any uncooked content
	 * 
	 * @return Override display name or empty string
	 */
	CONTENTBROWSERDATA_API FText GetFolderItemDisplayNameOverride(const FName InFolderPath, const FString& InFolderItemName, const bool bIsClassesFolder, const bool bIsCookedPath = false);

	CONTENTBROWSERDATA_API bool IsPrimaryAsset(const FAssetData& InAssetData);

	CONTENTBROWSERDATA_API bool IsPrimaryAsset(UObject* InObject);
}
