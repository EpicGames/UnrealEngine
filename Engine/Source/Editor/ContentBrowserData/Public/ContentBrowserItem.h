// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserItemData.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Templates/TypeHash.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ContentBrowserItem.generated.h"

#define UE_API CONTENTBROWSERDATA_API

enum class EAssetAccessSpecifier : uint8;
struct FContentBrowserItemPath;
class FAssetThumbnail;
class FContentBrowserMinimalItemData;
class IContentBrowserHideFolderIfEmptyFilter;
class UContentBrowserDataSource;
class UObject;
struct FAssetData;
struct FFrame;

/**
 * Representation of a Content Browser item.
 *
 * FContentBrowserItem is potentially a composite of multiple internal items (eg, combining equivalent folder items from different data sources),
 * and defers back to these internal items to provide its functionality (via the data source that owns each internal item).
 */
USTRUCT(BlueprintType)
struct FContentBrowserItem
{
	GENERATED_BODY()

public:
	using FItemDataArrayView = TArrayView<const FContentBrowserItemData>;
	
	/**
	 * Default constructor.
	 * Produces an item that is empty (IsValid() will return false).
	 */
	FContentBrowserItem() = default;

	/**
	 * Construct this composite item from the given internal item(s).
	 */
	UE_API explicit FContentBrowserItem(FContentBrowserItemData&& InItem);
	UE_API explicit FContentBrowserItem(const FContentBrowserItemData& InItem);
	UE_API explicit FContentBrowserItem(TArrayView<const FContentBrowserItemData> InItems);

	/**
	 * Copy support.
	 */
	FContentBrowserItem(const FContentBrowserItem&) = default;
	FContentBrowserItem& operator=(const FContentBrowserItem&) = default;

	/**
	 * Move support.
	 */
	FContentBrowserItem(FContentBrowserItem&&) = default;
	FContentBrowserItem& operator=(FContentBrowserItem&&) = default;

	/**
	 * Check to see whether this item is valid (contains at least one internal item).
	 */
	UE_API bool IsValid() const;

	/**
	 * Append the given item to this one, asserting if the combination isn't possible.
	 */
	UE_API void Append(const FContentBrowserItem& InOther);

	/**
	 * Attempt to append the given item to this one, providing error information if the combination isn't possible.
	 * @return True if the combination was possible, false otherwise.
	 */
	UE_API bool TryAppend(const FContentBrowserItem& InOther, FText* OutError = nullptr);

	/**
	 * Append the given item to this one, asserting if the combination isn't possible.
	 */
	UE_API void Append(const FContentBrowserItemData& InOther);

	/**
	 * Attempt to append the given item to this one, providing error information if the combination isn't possible.
	 * @return True if the combination was possible, false otherwise.
	 */
	UE_API bool TryAppend(const FContentBrowserItemData& InOther, FText* OutError = nullptr);

	/**
	 * Remove the given item from this one, asserting if the removal wasn't possible.
	 */
	UE_API void Remove(const FContentBrowserItem& InOther);

	/**
	 * Attempt to remove the given item from this one, providing error information if the removal wasn't possible.
	 * @return True if the removal was possible, false otherwise.
	 */
	UE_API bool TryRemove(const FContentBrowserItem& InOther, FText* OutError = nullptr);

	/**
	 * Remove the given item from this one, asserting if the removal wasn't possible.
	 */
	UE_API void Remove(const FContentBrowserItemData& InOther);

	/**
	 * Attempt to remove the given item from this one, providing error information if the removal wasn't possible.
	 * @return True if the removal was possible, false otherwise.
	 */
	UE_API bool TryRemove(const FContentBrowserItemData& InOther, FText* OutError = nullptr);

	/**
	 * Attempt to remove an item matching the given minimal representation from this one, providing error information if the removal wasn't possible.
	 * @return A valid item data (IsValid) if anything was removed, an invalid one otherwise. 
	 */
	UE_API FContentBrowserItemData TryRemove(const FContentBrowserMinimalItemData& InKey, FText* OutError = nullptr);

	/**
	 * Get the array of internal items that comprise this composite item.
	 */
	UE_API FItemDataArrayView GetInternalItems() const;

	/**
	 * Get the primary internal item from this composite item, if any.
	 */
	UE_API const FContentBrowserItemData* GetPrimaryInternalItem() const;

	/**
	 * Check to see whether this item is a folder.
	 * @note Equivalent to testing whether EContentBrowserItemFlags::Type_Folder is set on GetItemFlags().
	 */
	UE_API bool IsFolder() const;

	/**
	 * Check to see whether this item is a file.
	 * @note Equivalent to testing whether EContentBrowserItemFlags::Type_File is set on GetItemFlags().
	 */
	UE_API bool IsFile() const;

	/**
	 * Check to see whether this item is in a plugin.
	 * @note Equivalent to testing whether EContentBrowserItemFlags::Category_Plugin is set on GetItemFlags().
	 */
	UE_API bool IsInPlugin() const;

	/**
	 * Check if the item is representing a supported item
	 * The content browser can also display some unsupported asset
	 * @note Equivalent to testing whether EContentBrowserItemFlags::Misc_Unsupported is not set on GetItemFlags()
	 */
	UE_API bool IsSupported() const;

	/**
	 * Check to see whether this item is temporary.
	 * @note Equivalent to testing whether any of EContentBrowserItemFlags::Temporary_MASK is set on GetItemFlags().
	 */
	UE_API bool IsTemporary() const;

	/**
	 * Check to see whether this item is a display only folder.
	 * @note Equivalent to testing whether all of EContentBrowserItemFlags::Category_MASK are unset on GetItemFlags().
	 */
	UE_API bool IsDisplayOnlyFolder() const;

	/**
	 * Get the flags denoting basic state information for this item instance.
	 */
	UE_API EContentBrowserItemFlags GetItemFlags() const;

	/**
	 * Get the flags denoting the item type information for this item instance.
	 * @note Equivalent to applying EContentBrowserItemFlags::Type_MASK to GetItemFlags().
	 */
	UE_API EContentBrowserItemFlags GetItemType() const;

	/**
	 * Get the flags denoting the item category information for this item instance.
	 * @note Equivalent to applying EContentBrowserItemFlags::Category_MASK to GetItemFlags().
	 */
	UE_API EContentBrowserItemFlags GetItemCategory() const;

	/**
	 * Get the flags denoting the item temporary reason information for this item instance.
	 * @note Equivalent to applying EContentBrowserItemFlags::Temporary_MASK to GetItemFlags().
	 */
	UE_API EContentBrowserItemFlags GetItemTemporaryReason() const;

	/**
	 * Get the complete virtual path that uniquely identifies this item within its owner data source (eg, "/MyRoot/MyFolder/MyFile").
	 */
	UE_API FName GetVirtualPath() const;

	/**
	 * Gets the path that will not change based on toggling "Show All Folder", "Organize Folders" or other rules (eg,. "/Plugins").
	 */
	UE_API FName GetInvariantPath() const;

	/**
	 * Gets the internal path if it has one (eg,. "/Game").
	 */
	UE_API FName GetInternalPath() const;

	/**
	 * Get the leaf-name of this item (eg, "MyFile").
	 */
	UE_API FName GetItemName() const;

	/**
	 * Get the user-facing name of this item (eg, "MyFile").
	 */
	UE_API FText GetDisplayName() const;

	/**
	 * Query the EAssetAccessSpecifier of the item.
	 * 
	 * @param OutAssetAccessSpecifier The EAssetAccessSpecifier reference that will be filled.
	 *
	 * @return True if OutAssetAccessSpecifier was filled.
	 */
	UE_API bool GetItemAssetAccessSpecifier(EAssetAccessSpecifier& OutAssetAccessSpecifier) const;

	/**
	 * Query the item's ability to modify its EAssetAccessSpecifier.
	 *
	 * @return True if the item has an EAssetAccessSpecifier that can be modified.
	 */
	UE_API bool CanModifyItemAssetAccessSpecifier() const;

	/**
	 * Query the value of the given attribute on this item.
	 *
	 * @param InAttributeKey The name of the attribute to query.
	 * @param InIncludeMetaData True if we should also include any additional meta-data for the value.
	 * 
	 * @return The attribute value (test for validity).
	 */
	UE_API FContentBrowserItemDataAttributeValue GetItemAttribute(const FName InAttributeKey, const bool InIncludeMetaData = false) const;

	/**
	 * Query the values of all attributes on this item.
	 *
	 * @param InIncludeMetaData True if we should also include any additional meta-data for the values.
	 *
	 * @return Map of any attributes available on this item, along with their values.
	 */
	UE_API FContentBrowserItemDataAttributeValues GetItemAttributes(const bool InIncludeMetaData = false) const;

	/**
	 * Query the physical (on-disk) path of this item.
	 *
	 * @param OutDiskPath The string to fill with the result.
	 *
	 * @return True if the physical path was found and the result filled, false otherwise.
	 */
	UE_API bool GetItemPhysicalPath(FString& OutDiskPath) const;

	/**
	 * Query whether this item is considered dirty (ie, has unsaved changes).
	 * @return True if the item is considered dirty, false otherwise.
	 */
	UE_API bool IsDirty() const;

	/**
	 * Query whether this item is can be edited, optionally providing error information if it cannot.
	 *
	 * @param OutErrorMessage Optional error message to fill on failure.
	 *
	 * @return True if the item can be edited, false otherwise.
	 */
	UE_API bool CanEdit(FText* OutErrorMsg = nullptr) const;

	/**
	 * Attempt to open this item for editing.
	 * @return True if the item was opened for editing, false otherwise.
	 */
	UE_API bool Edit() const;

	/**
	 * Query whether this item is can be previewed, optionally providing error information if it cannot.
	 *
	 * @param OutErrorMessage Optional error message to fill on failure.
	 *
	 * @return True if the item can be previewed, false otherwise.
	 */
	UE_API bool CanPreview(FText* OutErrorMsg = nullptr) const;

	/**
	 * Query whether the given item is can be viewed (a read-only asset editor), optionally providing error information if it cannot.
	 *
	 * @param OutErrorMessage Optional error message to fill on failure.
	 *
	 * @return True if the item can be viewed in a read-only editor, false otherwise.
	 */
	UE_API bool CanView(FText* OutErrorMsg = nullptr) const;

	/**
	 * Attempt to preview this item.
	 * @return True if the item was previewed, false otherwise.
	 */
	UE_API bool Preview() const;

	/**
	 * Query whether this item is can be duplicated, optionally providing error information if it cannot.
	 *
	 * @param OutErrorMessage Optional error message to fill on failure.
	 *
	 * @return True if the item can be duplicated, false otherwise.
	 */
	UE_API bool CanDuplicate(FText* OutErrorMsg = nullptr) const;

	/*
	 * Attempt to begin the process of asynchronously duplicating this item, populating a temporary item that can be finalized or canceled by the user.
	 * @note This duplicates the item at its current path and assigns it a default unique name.
	 *
	 * @return The temporary context of the duplicate item (test for validity).
	 */
	UE_API FContentBrowserItemDataTemporaryContext Duplicate() const;

	/**
	 * Query whether this item is can be saved, optionally providing error information if it cannot.
	 *
	 * @param InSaveFlags Flags controlling the rules behind when the item should be saved.
	 * @param OutErrorMessage Optional error message to fill on failure.
	 *
	 * @return True if the item can be saved, false otherwise.
	 */
	UE_API bool CanSave(const EContentBrowserItemSaveFlags InSaveFlags, FText* OutErrorMsg = nullptr) const;

	/**
	 * Attempt to save this item.
	 *
	 * @param InSaveFlags Flags controlling the rules behind when the item should be saved.
	 *
	 * @return True if the item was saved, false otherwise.
	 */
	UE_API bool Save(const EContentBrowserItemSaveFlags InSaveFlags) const;

	/**
	 * Query whether this item is can be deleted, optionally providing error information if it cannot.
	 *
	 * @param OutErrorMessage Optional error message to fill on failure.
	 *
	 * @return True if the item can be deleted, false otherwise.
	 */
	UE_API bool CanDelete(FText* OutErrorMsg = nullptr) const;

	/**
	 * Attempt to delete this item.
	 * @return True if the item was deleted, false otherwise.
	 */
	UE_API bool Delete() const;

	/**
	 * Query whether this item is can be renamed, optionally providing error information if it cannot.
	 *
	 * @param InNewName Optional name that will be tested for validity alongside the general ability to rename this item.
	 * @param OutErrorMessage Optional error message to fill on failure.
	 *
	 * @return True if the item can be renamed, false otherwise.
	 */
	UE_API bool CanRename(const FString* InNewName, FText* OutErrorMsg = nullptr) const;

	/**
	 * Query whether this item is can be renamed, optionally providing error information if it cannot.
	 *
	 * @param InNewName Optional name that will be tested for validity alongside the general ability to rename this item.
	 * @param HideFolderIfEmptyFilter Current empty folder filter used to permit rename operations to hidden folders.
	 * @param OutErrorMsg Optional error message to fill on failure.
	 *
	 * @return True if the item can be renamed, false otherwise.
	 */
	UE_API bool CanRename(const FString* InNewName, const IContentBrowserHideFolderIfEmptyFilter* HideFolderIfEmptyFilter, FText* OutErrorMsg = nullptr) const;

	/**
	 * Attempt to rename this item.
	 *
	 * @param InNewName The new name to give the item.
	 * @param OutNewItem The item after it has been renamed.
	 *
	 * @return True if the item was renamed, false otherwise.
	 */
	UE_API bool Rename(const FString& InNewName, FContentBrowserItem* OutNewItem = nullptr) const;

	/**
	 * Query whether this item is can be copied, optionally providing error information if it cannot.
	 *
	 * @param InDestPath The virtual path that the item will be copied to.
	 * @param OutErrorMessage Optional error message to fill on failure.
	 *
	 * @return True if the item can be copied, false otherwise.
	 */
	UE_API bool CanCopy(const FName InDestPath, FText* OutErrorMsg = nullptr) const;

	/**
	 * Attempt to copy this item to the given virtual path.
	 *
	 * @param InDestPath The virtual path that the item will be copied to.
	 *
	 * @return True if the item was copied, false otherwise.
	 */
	UE_API bool Copy(const FName InDestPath) const;

	/**
	 * Query whether this item is can be moved, optionally providing error information if it cannot.
	 *
	 * @param InDestPath The virtual path that the item will be moved to.
	 * @param OutErrorMessage Optional error message to fill on failure.
	 *
	 * @return True if the item can be moved, false otherwise.
	 */
	UE_API bool CanMove(const FName InDestPath, FText* OutErrorMsg = nullptr) const;

	/**
	 * Attempt to move this item to the given virtual path.
	 *
	 * @param InDestPath The virtual path that the item will be moved to.
	 *
	 * @return True if the item was moved, false otherwise.
	 */
	UE_API bool Move(const FName InDestPath) const;

	/**
	 * Attempt to append any path references for this item to the given string.
	 * @note Used when copying item references to the clipboard.
	 *
	 * @param InOutStr The string to append to (LINE_TERMINATOR delimited).
	 *
	 * @return True if references were appended, false otherwise.
	 */
	UE_API bool AppendItemReference(FString& InOutStr) const;

	/**
	 * Attempt to append any object path for this item to the given string.
	 * @note Used when copying item object path to the clipboard.
	 *
	 * @param InOutStr The string to append to (LINE_TERMINATOR delimited).
	 *
	 * @return True if references were appended, false otherwise.
	 */
	UE_API bool AppendItemObjectPath(FString& InOutStr) const;

	/**
	 * Attempt to append any package name for this item to the given string.
	 * @note Used when copying item package name to the clipboard.
	 *
	 * @param InOutStr The string to append to (LINE_TERMINATOR delimited).
	 *
	 * @return True if references were appended, false otherwise.
	 */
	UE_API bool AppendItemPackageName(FString& InOutStr) const;

	/**
	 * Attempt to update the thumbnail associated with this item.
	 *
	 * @param InThumbnail The thumbnail to update.
	 *
	 * @return True if the thumbnail was updated, false otherwise.
	 */
	UE_API bool UpdateThumbnail(FAssetThumbnail& InThumbnail) const;

	/**
	 * Attempt to retrieve the identifier that should be used when storing a reference to this item within a collection.
	 *
	 * @param InOutStr The collection ID to fill.
	 *
	 * @return True if the ID was retrieved, false otherwise.
	 */
	UE_API bool TryGetCollectionId(FSoftObjectPath& OutCollectionId) const;

	/**
	 * Attempt to retrieve the package path associated with this item.
	 * @note This exists to allow the Content Browser to interface with external callbacks that only operate on package paths and should ideally be avoided for new code.
	 *
	 * @param InOutStr The package path to fill.
	 *
	 * @return True if the package path was retrieved, false otherwise.
	 */
	UE_API bool Legacy_TryGetPackagePath(FName& OutPackagePath) const;

	/**
	 * Attempt to retrieve the asset data associated with this item.
	 * @note This exists to allow the Content Browser to interface with external callbacks that only operate on asset data and should ideally be avoided for new code.
	 *
	 * @param InOutStr The asset data to fill.
	 *
	 * @return True if the asset data was retrieved, false otherwise.
	 */
	UE_API bool Legacy_TryGetAssetData(FAssetData& OutAssetData) const;

	/**
	 * Attempt to retrieve the source asset path of an alias.
	 * 
	 * @param OutObjectPath The soft object path to fill with the result
	 * @return True if @code this @endcode is an alias and a path was found, false otherwise
	 */
	UE_API bool GetItemAliasedPath(FSoftObjectPath& OutObjectPath) const;

	/**
	 * Creates a ContentBrowserItemPath, potentially looking up internal paths.
	 * @return The Item Path.
	 */
	UE_API FContentBrowserItemPath GetItemPath() const;

private:
	/**
	 * Array of internal items that comprise this composite item.
	 * @note Optimized to avoid allocation for the file case, where each composite item has a single internal item.
	 */
	TArray<FContentBrowserItemData, TInlineAllocator<1>> ItemDataArray;
};

// TODO: Script API exposure
UCLASS(MinimalAPI)
class UContentBrowserItemLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
	
public:
	UFUNCTION(BlueprintPure, Category="ContentBrowser", meta=(ScriptMethod))
	static UE_API bool IsFolder(const FContentBrowserItem& Item);

	UFUNCTION(BlueprintPure, Category="ContentBrowser", meta=(ScriptMethod))
	static UE_API bool IsFile(const FContentBrowserItem& Item);

	UFUNCTION(BlueprintPure, Category="ContentBrowser", meta=(ScriptMethod))
	static UE_API FName GetVirtualPath(const FContentBrowserItem& Item);

	UFUNCTION(BlueprintPure, Category="ContentBrowser", meta=(ScriptMethod))
	static UE_API FText GetDisplayName(const FContentBrowserItem& Item);
};

/**
 * Context for asynchronous item creation (for new items or duplicating existing ones).
 * Data sources will return one of these (hosting a temporary item with any required context) when they want to begin the creation or duplication of an item item.
 * UI involved in this creation flow should call ValidateItem for any name changes, as well as before calling FinalizeItem. Once finalized the temporary item should be replaced with the real one.
 */
class FContentBrowserItemTemporaryContext
{
public:
	/**
	 * Check to see whether this context is valid (has been set to a temporary item).
	 */
	UE_API bool IsValid() const;

	/**
	 * Append the given context to this one.
	 */
	UE_API void AppendContext(FContentBrowserItemDataTemporaryContext&& InContext);

	/**
	 * Get the data representing the temporary item instance.
	 */
	UE_API const FContentBrowserItem& GetItem() const;

	/**
	 * Invoke the delegate used to validate that the proposed item name is valid.
	 * @note Returns True if no validation delegate is bound.
	 */
	UE_API bool ValidateItem(const FString& InProposedName, FText* OutErrorMsg = nullptr) const;

	/**
	 * Invoke the delegate used to finalize the creation of the temporary item, and return the real item.
	 */
	UE_API FContentBrowserItem FinalizeItem(const FString& InProposedName, FText* OutErrorMsg = nullptr) const;

private:
	/**
	 * Data representing the temporary item instance.
	 */
	FContentBrowserItem Item;

	/**
	 * Array of internal contexts that comprise this composite context.
	 * @note Optimized to avoid allocation for the file case, where each composite context has a single internal context.
	 */
	TArray<FContentBrowserItemDataTemporaryContext, TInlineAllocator<1>> ItemDataContextArray;
};

/**
 * Minimal representation of a FContentBrowserItem instance that can be used as a map key.
 */
class FContentBrowserItemKey : public FContentBrowserItemDataKey
{
public:
	/**
	 * Default constructor.
	 * Produces a key that is empty.
	 */
	FContentBrowserItemKey() = default;

	/**
	 * Construct this key from the given item.
	 */
	UE_API explicit FContentBrowserItemKey(const FContentBrowserItem& InItem);
	UE_API explicit FContentBrowserItemKey(const FContentBrowserItemData& InItemData);
	
	/**
	 * Construct this key from the given item type, virtual path, and data source.
	 */
	UE_API FContentBrowserItemKey(EContentBrowserItemFlags InItemType, FName InPath, const UContentBrowserDataSource* InDataSource);

	/**
	 * Equality support.
	 */
	bool operator==(const FContentBrowserItemKey& InOther) const
	{
		return FContentBrowserItemDataKey::operator==(InOther)
			&& DataSource == InOther.DataSource;
	}

	/**
	 * Inequality support.
	 */
	bool operator!=(const FContentBrowserItemKey& InOther) const
	{
		return !(*this == InOther);
	}

	/**
	 * Get the hash of the given instance.
	 */
	friend inline uint32 GetTypeHash(const FContentBrowserItemKey& InKey)
	{
		return HashCombine(GetTypeHash(static_cast<const FContentBrowserItemDataKey&>(InKey)), GetTypeHash(InKey.DataSource));
	}

protected:
	/**
	 * Internal common function shared by the FContentBrowserItem and FContentBrowserItemData constructors.
	 */
	UE_API void Initialize(const FContentBrowserItemData& InItemData);

	/** A pointer to the data source that manages the thing represented by this key */
	const UContentBrowserDataSource* DataSource = nullptr;
};

/**
 * Type describing an update to an item.
 * @note This is only really useful for script exposure, as it's more efficient for C++ to use FContentBrowserItemDataUpdate directly
 */
class FContentBrowserItemUpdate
{
public:
	/**
	 * Create an item update from an internal update event.
	 */
	UE_API explicit FContentBrowserItemUpdate(const FContentBrowserItemDataUpdate& InItemDataUpdate);

	/**
	 * Get the value denoting the types of updates that can be emitted for an item.
	 */
	UE_API EContentBrowserItemUpdateType GetUpdateType() const;

	/**
	 * Get the item data for the update.
	 */
	UE_API const FContentBrowserItem& GetItemData() const;

	/**
	 * Get the previous virtual path (UpdateType == Moved).
	 */
	UE_API FName GetPreviousVirtualPath() const;

private:
	/** Value denoting the types of updates that can be emitted for an item */
	EContentBrowserItemUpdateType UpdateType;

	/** Item data for the update */
	FContentBrowserItem ItemData;

	/** Previous virtual path (UpdateType == Moved) */
	FName PreviousVirtualPath;
};

#undef UE_API
