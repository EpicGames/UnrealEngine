// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/StringBuilder.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API CONTENTBROWSERDATA_API

enum class EAssetAccessSpecifier : uint8;
class FContentBrowserMinimalItemData;
class IContentBrowserItemDataSink;
class UContentBrowserDataSource;

/** Flags denoting basic state information for an item instance */
enum class EContentBrowserItemFlags : uint16
{
	/** No flags */
	None = 0,

	/** Flags denoting the item type (mutually exclusive) */
	Type_Folder = 1<<0,
	Type_File = 1<<1,
	Type_MASK = Type_Folder | Type_File,

	/** Flags denoting the item category */
	Category_Asset = 1<<2,
	Category_Class = 1<<3,
	Category_Collection = 1<<4,
	Category_Plugin = 1 << 5, // Plugin content or classes - note that folders containing plugins do not have this flag set 
	Category_Misc = 1<<6,
	Category_MASK = Category_Asset | Category_Class | Category_Collection | Category_Plugin | Category_Misc,

	/** Flags denoting additional information for temporary items (mutually exclusive) */
	Temporary_Creation = 1<<7,
	Temporary_Duplication = 1<<8,
	Temporary_MASK = Temporary_Creation | Temporary_Duplication,

	/** Flag to mark the item as showing something that is unsupported */
	Misc_Unsupported = 1 << 9,
};
ENUM_CLASS_FLAGS(EContentBrowserItemFlags);

/** Flags denoting the save behavior of items */
enum class EContentBrowserItemSaveFlags : uint8
{
	/** No flags */
	None = 0,
	/** Save the item only if it is considered dirty */
	SaveOnlyIfDirty = 1<<0,
	/** Save the item only if it is currently loaded (will load the item if required when this flag isn't set) */
	SaveOnlyIfLoaded = 1<<1,
};
ENUM_CLASS_FLAGS(EContentBrowserItemSaveFlags);

/** Enum denoting the types of values that an item attribute can store */
enum class EContentBrowserItemDataAttributeValueType : uint8
{
	/** No value is set */
	None,
	/** An FString value is set */
	String,
	/** An FName value is set */
	Name,
	/** An FText value is set */
	Text,
};

/** Enum denoting the types of updates that can be emitted for an item */
enum class EContentBrowserItemUpdateType : uint8
{
	/** This item was newly added */
	Added,
	/** This item already existed, and has been modified */
	Modified,
	/** This item already existed, and has been moved to a new location or renamed */
	Moved,
	/** This item existed and has been deleted */
	Removed,
	/** This item existed and has been unmounted */
	Unmounted,
};

/**
 * Interface used to store any data source defined payload data that is required to operate on the underlying thing that a Content Browser item represents.
 * @note This is deliberately opaque as only the owner data source should know what it is.
 */
class IContentBrowserItemDataPayload
{
public:
	virtual ~IContentBrowserItemDataPayload() = default;
};

/**
 * The primitive data that represents an internal Content Browser item, as defined and managed by a Content Browser data source.
 *
 * FContentBrowserItemData itself is a concrete type, so extensibility is handled via the IContentBrowserItemDataPayload interface, which can be
 * used to store any data source defined payload data that is required to operate on the underlying thing that this item represents.
 *
 * FContentBrowserItemData has no real functionality, and relies on passing itself back into the correct data source instance when asked to perform 
 * actions or validation.
 *
 * @note This is the lower-level version of FContentBrowserItem, and is used internally by data sources. External code is more likely to use 
 * FContentBrowserItem directly, as that type can composite multiple internal items together (eg, combining equivalent folder items from different data sources).
 *
 * @see UContentBrowserDataSource.
 */
class FContentBrowserItemData final
{
public:
	/**
	 * Default constructor.
	 * Produces an item that is empty (IsValid() will return false).
	 */
	FContentBrowserItemData() = default;

	/**
	 * Create an internal Content Browser item.
	 *
	 * @param InOwnerDataSource A pointer to the data source that manages the thing represented by this item. This is
	 * usually set, but may be null in rare circumstances (ie, creating a dummy placeholder item with no owner).
	 * @param InItemFlags Flags denoting basic state information for this item instance.
	 * @param InVirtualPath The complete virtual path that uniquely identifies this item within its owner data source
	 * (eg, "/MyRoot/MyFolder/MyFile").
	 * @param InItemName The leaf-name of this item (eg, "MyFile").
	 * @param InDisplayNameOverride The user-facing name of this item (eg, "MyFile"). This will be lazily set to
	 * InItemName if no override is provided.
	 * @param InPayload Any data source defined payload data for this item.
	 * @param InInternalPath Internal path if any (some items such as "virtual folders" may not have an internal path)
	 */
	UE_API FContentBrowserItemData(
		UContentBrowserDataSource* InOwnerDataSource,
		EContentBrowserItemFlags InItemFlags,
		FName InVirtualPath,
		FName InItemName,
		FText InDisplayNameOverride,
		TSharedPtr<const IContentBrowserItemDataPayload> InPayload,
		FName InInternalPath);

	/**
	 * Copy support.
	 */
	FContentBrowserItemData(const FContentBrowserItemData&) = default;
	FContentBrowserItemData& operator=(const FContentBrowserItemData&) = default;

	/**
	 * Move support.
	 */
	FContentBrowserItemData(FContentBrowserItemData&&) = default;
	FContentBrowserItemData& operator=(FContentBrowserItemData&&) = default;

	/**
	 * Comparison support.
	 */
	UE_API bool operator==(const FContentBrowserItemData& InOther) const;
	UE_API bool operator!=(const FContentBrowserItemData& InOther) const;

	/**
	 * Check to see whether this item is valid (is either a folder or a file).
	 */
	UE_API bool IsValid() const;

	/**
	 * Check to see whether this item is a folder.
	 * @note Equivalent to testing whether EContentBrowserItemFlags::Type_Folder is set on GetItemFlags().
	 */
	UE_API bool IsFolder() const;

	/**
	 * Check to see whether this item is a file.
	 * @note Equivalent to testing whether EContentBrowserItemFlags::Type_File is set on GetItemFlags().
	 */
	UE_API bool IsFile() const;

	/**
	 * Check to see whether this item is in a plugin. Folders and files inside plugins return true, folders which
	 * contain and organize plugins do not.
	 */
	UE_API bool IsPlugin() const;

	/**
	 * Check if the item is representing a supported item
	 * The content browser can also display some unsupported asset
	 * @note Equivalent to testing whether EContentBrowserItemFlags::Misc_Unsupported is not set on GetItemFlags()
	 */
	UE_API bool IsSupported() const;

	/**
	 * Check to see whether this item is temporary.
	 * @note Equivalent to testing whether any of EContentBrowserItemFlags::Temporary_MASK is set on GetItemFlags().
	 */
	UE_API bool IsTemporary() const;

	/**
	 * Check to see whether this item is a display only folder.
	 * @note Equivalent to testing whether all of EContentBrowserItemFlags::Category_MASK are unset on GetItemFlags().
	 */
	UE_API bool IsDisplayOnlyFolder() const;

	/**
	 * Get the pointer to the data source that manages the thing represented by this item.
	 * @note This is usually set for valid items, but may be null in rare circumstances (ie, creating a dummy placeholder item with no owner).
	 */
	UE_API UContentBrowserDataSource* GetOwnerDataSource() const;

	/**
	 * Get the flags denoting basic state information for this item instance.
	 */
	UE_API EContentBrowserItemFlags GetItemFlags() const;

	/**
	 * Get the flags denoting the item type information for this item instance.
	 * @note Equivalent to applying EContentBrowserItemFlags::Type_MASK to GetItemFlags().
	 */
	UE_API EContentBrowserItemFlags GetItemType() const;

	/**
	 * Get the flags denoting the item category information for this item instance.
	 * @note Equivalent to applying EContentBrowserItemFlags::Category_MASK to GetItemFlags().
	 */
	UE_API EContentBrowserItemFlags GetItemCategory() const;

	/**
	 * Get the flags denoting the item temporary reason information for this item instance.
	 * @note Equivalent to applying EContentBrowserItemFlags::Temporary_MASK to GetItemFlags().
	 */
	UE_API EContentBrowserItemFlags GetItemTemporaryReason() const;

	/**
	 * Get the complete virtual path that uniquely identifies this item within its owner data source (eg, "/All/MyRoot/MyFolder/MyFile").
	 */
	UE_API FName GetVirtualPath() const;

	/**
	 * Get the complete invariant path that uniquely identifies this item within its owner data source (eg, "/MyRoot/MyFolder/MyFile").
	 * This path will be the same regardless of the options being toggled 'Show All Folder' or 'Organize Folders'
	 */
	UE_API FName GetInvariantPath() const;

	/**
	 * Get the complete internal path that uniquely identifies this item within its owner data source if it has one (eg, "/MyRoot/MyFolder/MyFile").
	 */
	UE_API FName GetInternalPath() const;

	/**
	 * Get the leaf-name of this item (eg, "MyFile").
	 */
	UE_API FName GetItemName() const;

	/**
	 * Get the user-facing name of this item (eg, "MyFile").
	 */
	UE_API FText GetDisplayName() const;

	/**
	 * Query the EAssetAccessSpecifier of the item.
	 *
	 * @param OutAssetAccessSpecifier The EAssetAccessSpecifier reference that will be filled.
	 *
	 * @return True if OutAssetAccessSpecifier was filled.
	 */
	UE_API bool GetItemAssetAccessSpecifier(EAssetAccessSpecifier& OutAssetAccessSpecifier) const;

	/**
	 * Query the item's ability to modify its EAssetAccessSpecifier.
	 *
	 * @return True if the item has an EAssetAccessSpecifier that can be modified.
	 */
	UE_API bool CanModifyItemAssetAccessSpecifier() const;

	/**
	 * Get any data source defined payload data for this item.
	 */
	UE_API TSharedPtr<const IContentBrowserItemDataPayload> GetPayload() const;

private:
	/** A pointer to the data source that manages the thing represented by this item. This is usually set for valid items, but may be null in rare circumstances (ie, creating a dummy placeholder item with no owner) */
	TWeakObjectPtr<UContentBrowserDataSource> OwnerDataSource;

	/** Flags denoting basic state information for this item instance */
	EContentBrowserItemFlags ItemFlags = EContentBrowserItemFlags::None;

	/** The complete virtual path that uniquely identifies this item within its owner data source (eg, "/MyRoot/MyFolder/MyFile") */
	FName VirtualPath;

	/** The leaf-name of this item (eg, "MyFile") */
	FName ItemName;

	/** Internal path used to identify item within the owner data source */
	FName InternalPath;

	/** The user-facing name of this item (eg, "MyFile") */
	FText CachedDisplayName;

	/** Any data source defined payload data for this item */
	TSharedPtr<const IContentBrowserItemDataPayload> Payload;
};

/**
 * Type describing the meta-data associated with an item attribute value.
 * This provides additional context and information that can be used to interpret and display the value.
 */
class FContentBrowserItemDataAttributeMetaData
{
public:
	/**
	 * Constructor.
	 */
	FContentBrowserItemDataAttributeMetaData() = default;

	/**
	 * Copy support.
	 */
	FContentBrowserItemDataAttributeMetaData(const FContentBrowserItemDataAttributeMetaData&) = default;
	FContentBrowserItemDataAttributeMetaData& operator=(const FContentBrowserItemDataAttributeMetaData&) = default;

	/**
	 * Move support.
	 */
	FContentBrowserItemDataAttributeMetaData(FContentBrowserItemDataAttributeMetaData&&) = default;
	FContentBrowserItemDataAttributeMetaData& operator=(FContentBrowserItemDataAttributeMetaData&&) = default;

	/** The kind of data represented by this attribute value */
	UObject::FAssetRegistryTag::ETagType AttributeType = UObject::FAssetRegistryTag::TT_Hidden;

	/** Flags giving hints at how to display this attribute value in the UI (see ETagDisplay) */
	uint32 DisplayFlags = UObject::FAssetRegistryTag::TD_None;

	/** Resolved display name of the associated attribute */
	FText DisplayName;
	
	/** Optional tooltip of the associated attribute */
	FText TooltipText;
	
	/** Optional suffix to apply to values of the associated attribute in the UI */
	FText Suffix;

	/** True if this attribute value is considered "important" in the UI */
	bool bIsImportant = false;
};

/**
 * Type describing the value of an item attribute.
 * Internally this is optimized to store the value as either an FString, FName, or FText.
 */
class FContentBrowserItemDataAttributeValue
{
public:
	/**
	 * Default constructor.
	 * Produces an attribute value that is empty (IsValid() will return false).
	 */
	FContentBrowserItemDataAttributeValue() = default;

	/**
	 * Construct this attribute value from the given string.
	 */
	UE_API explicit FContentBrowserItemDataAttributeValue(const TCHAR* InStr);
	UE_API explicit FContentBrowserItemDataAttributeValue(const FString& InStr);
	UE_API explicit FContentBrowserItemDataAttributeValue(FString&& InStr);

	/**
	 * Construct this attribute value from the given name.
	 */
	UE_API explicit FContentBrowserItemDataAttributeValue(const FName InName);

	/**
	 * Construct this attribute value from the given text.
	 */
	UE_API explicit FContentBrowserItemDataAttributeValue(FText InText);

	/**
	 * Copy support.
	 */
	FContentBrowserItemDataAttributeValue(const FContentBrowserItemDataAttributeValue&) = default;
	FContentBrowserItemDataAttributeValue& operator=(const FContentBrowserItemDataAttributeValue&) = default;

	/**
	 * Move support.
	 */
	FContentBrowserItemDataAttributeValue(FContentBrowserItemDataAttributeValue&&) = default;
	FContentBrowserItemDataAttributeValue& operator=(FContentBrowserItemDataAttributeValue&&) = default;

	/**
	 * Check to see whether this attribute value is valid (has been set to an internal value).
	 */
	UE_API bool IsValid() const;

	/**
	 * Reset this attribute value to its empty default state.
	 */
	UE_API void Reset();

	/**
	 * Get the value denoting what type of internal value is stored within this attribute value.
	 */
	UE_API EContentBrowserItemDataAttributeValueType GetValueType() const;
	
	/**
	 * Get this attribute value as the requested type, converting its internal value to the given type if required.
	 * @note This function is specialized for FString, FName, and FText, with the generic implementation requiring a LexFromString implementation for the requested type.
	 */
	template <typename ValueAttrType>
	ValueAttrType GetValue() const;

	/**
	 * Set this attribute value from the given type, converting it to a type we can store internally if required.
	 * @note This function is overloaded for FString, FName, and FText, with the generic implementation requiring a LexToString implementation for the given type.
	 * @note This function will reset the type during the set, so will also clear any currently associated meta-data.
	 */
	template <typename ValueAttrType>
	void SetValue(ValueAttrType InValue);
	UE_API void SetValue(const TCHAR* InStr);
	UE_API void SetValue(const FString& InStr);
	UE_API void SetValue(FString&& InStr);
	UE_API void SetValue(const FName InName);
	UE_API void SetValue(FText InText);

	/**
	 * Get this attribute value as a string view, converting its internal value using the scratch buffer if required (if ValueType == Name).
	 */
	UE_API FStringView GetValueStringView(FStringBuilderBase& ScratchBuffer) const;

	/**
	 * Get the internal string value.
	 * @note Only valid when ValueType == String.
	 */
	UE_API const FString& GetValueString() const;

	/**
	 * Get the internal name value.
	 * @note Only valid when ValueType == Name.
	 */
	UE_API FName GetValueName() const;

	/**
	 * Get the internal text value.
	 * @note Only valid when ValueType == Name.
	 */
	UE_API FText GetValueText() const;

	/**
	 * Get the meta-data associated with this attribute value.
	 */
	UE_API const FContentBrowserItemDataAttributeMetaData& GetMetaData() const;

	/**
	 * Set the meta-data associated with this attribute value.
	 */
	UE_API void SetMetaData(const FContentBrowserItemDataAttributeMetaData& InMetaData);
	UE_API void SetMetaData(FContentBrowserItemDataAttributeMetaData&& InMetaData);

private:
	/** Value denoting what type of internal value is stored within this attribute value */
	EContentBrowserItemDataAttributeValueType ValueType = EContentBrowserItemDataAttributeValueType::None;

	/** Internal string value */
	FString StrValue;

	/** Internal name value */
	FName NameValue;

	/** Internal text value */
	FText TextValue;

	/** Meta-data associated with this attribute value */
	TSharedPtr<const FContentBrowserItemDataAttributeMetaData> MetaData;
};

template <typename ValueAttrType>
inline ValueAttrType FContentBrowserItemDataAttributeValue::GetValue() const
{
	TStringBuilder<FName::StringBufferSize> NameStr;

	const TCHAR* StrPtr = nullptr;
	switch (ValueType)
	{
	case EContentBrowserItemDataAttributeValueType::String:
		StrPtr = *StrValue;
		break;

	case EContentBrowserItemDataAttributeValueType::Name:
		NameValue.ToString(NameStr);
		StrPtr = *NameStr;
		break;

	case EContentBrowserItemDataAttributeValueType::Text:
		StrPtr = *TextValue.ToString();
		break;
	}

	ValueAttrType Result = ValueAttrType();
	if (StrPtr)
	{
		LexFromString(Result, StrPtr);
	}
	return Result;
}

template <>
inline FString FContentBrowserItemDataAttributeValue::GetValue<FString>() const
{
	switch (ValueType)
	{
	case EContentBrowserItemDataAttributeValueType::String:
		return StrValue;

	case EContentBrowserItemDataAttributeValueType::Name:
		return NameValue.ToString();

	case EContentBrowserItemDataAttributeValueType::Text:
		return TextValue.ToString();
	}

	return FString();
}

template <>
inline FText FContentBrowserItemDataAttributeValue::GetValue<FText>() const
{
	switch (ValueType)
	{
	case EContentBrowserItemDataAttributeValueType::String:
		return FText::FromString(StrValue);

	case EContentBrowserItemDataAttributeValueType::Name:
		return FText::FromName(NameValue);

	case EContentBrowserItemDataAttributeValueType::Text:
		return TextValue;
	}

	return FText();
}

template <>
inline FName FContentBrowserItemDataAttributeValue::GetValue<FName>() const
{
	switch (ValueType)
	{
	case EContentBrowserItemDataAttributeValueType::String:
		return FName(*StrValue);

	case EContentBrowserItemDataAttributeValueType::Name:
		return NameValue;

	case EContentBrowserItemDataAttributeValueType::Text:
		return FName(*TextValue.ToString());
	}

	return FName();
}

template <typename ValueAttrType>
void FContentBrowserItemDataAttributeValue::SetValue(ValueAttrType InValue)
{
	SetValue(LexToString(InValue));
}

typedef TMap<FName, FContentBrowserItemDataAttributeValue> FContentBrowserItemDataAttributeValues;

/**
 * Context for asynchronous item creation (for new items or duplicating existing ones).
 * Data sources will return one of these (hosting a temporary item with any required context) when they want to begin the creation or duplication of an item item.
 * UI involved in this creation flow should call ValidateItem for any name changes, as well as before calling FinalizeItem. Once finalized the temporary item should be replaced with the real one.
 */
class FContentBrowserItemDataTemporaryContext
{
public:
	/** Delegate used to validate that the proposed item name is valid */
	DECLARE_DELEGATE_RetVal_ThreeParams(bool, FOnValidateItem, const FContentBrowserItemData& /*InItemData*/, const FString& /*InProposedName*/, FText* /*OutErrorMsg*/);
	
	/** Delegate used to finalize the creation of the temporary item, and return the real item */
	DECLARE_DELEGATE_RetVal_ThreeParams(FContentBrowserItemData, FOnFinalizeItem, const FContentBrowserItemData& /*InItemData*/, const FString& /*InProposedName*/, FText* /*OutErrorMsg*/);

	/**
	 * Default constructor.
	 * Produces a context that is empty (IsValid() will return false).
	 */
	FContentBrowserItemDataTemporaryContext() = default;

	/**
	 * Create a context instance.
	 *
	 * @param InItemData Data representing the temporary item instance.
	 * @param InOnValidateItem Delegate used to validate that the proposed item name is valid (optional).
	 * @param InOnFinalizeItem Delegate used to finalize the creation of the temporary item, and return the real item.
	 */
	UE_API FContentBrowserItemDataTemporaryContext(FContentBrowserItemData&& InItemData, FOnValidateItem InOnValidateItem, FOnFinalizeItem InOnFinalizeItem);

	/**
	 * Copy support.
	 */
	FContentBrowserItemDataTemporaryContext(const FContentBrowserItemDataTemporaryContext&) = default;
	FContentBrowserItemDataTemporaryContext& operator=(const FContentBrowserItemDataTemporaryContext&) = default;

	/**
	 * Move support.
	 */
	FContentBrowserItemDataTemporaryContext(FContentBrowserItemDataTemporaryContext&&) = default;
	FContentBrowserItemDataTemporaryContext& operator=(FContentBrowserItemDataTemporaryContext&&) = default;

	/**
	 * Check to see whether this context is valid (has been set to a temporary item).
	 */
	UE_API bool IsValid() const;

	/**
	 * Get the data representing the temporary item instance.
	 */
	UE_API const FContentBrowserItemData& GetItemData() const;

	/**
	 * Invoke the delegate used to validate that the proposed item name is valid.
	 * @note Returns True if no validation delegate is bound.
	 */
	UE_API bool ValidateItem(const FString& InProposedName, FText* OutErrorMsg = nullptr) const;

	/**
	 * Invoke the delegate used to finalize the creation of the temporary item, and return the real item.
	 */
	UE_API FContentBrowserItemData FinalizeItem(const FString& InProposedName, FText* OutErrorMsg = nullptr) const;

private:
	/** Data representing the temporary item instance */
	FContentBrowserItemData ItemData;

	/** Delegate used to validate that the proposed item name is valid (optional) */
	FOnValidateItem OnValidateItem;

	/** Delegate used to finalize the creation of the temporary item, and return the real item */
	FOnFinalizeItem OnFinalizeItem;
};

/**
 * Minimal representation of a FContentBrowserItemData instance for comparison purposes.
 * @note Unlike FContentBrowserItemDataKey this does consider the data source, and unlike FContentBrowserItemKey it considers the data source for folder items.
 *  It can be used to identify a FContentBrowserItemData for removal from an FContentBrowserItem
 */
class FContentBrowserMinimalItemData
{
public:
	UE_API explicit FContentBrowserMinimalItemData(const FContentBrowserItemData& InItemData);
	UE_API FContentBrowserMinimalItemData(EContentBrowserItemFlags InItemType, FName InVirtualPath, const UContentBrowserDataSource* InSource);

	bool operator==(const FContentBrowserMinimalItemData& Other) const
	{
		return !(*this != Other);
	}
	bool operator!=(const FContentBrowserMinimalItemData& Other) const
	{
		return ItemType != Other.ItemType || VirtualPath != Other.VirtualPath || DataSource != Other.DataSource;
	}

	/** Return a string representation for debugging. */
	UE_API FString ToString() const;
	
	EContentBrowserItemFlags GetItemType() const
	{
		return ItemType;
	} 

	FName GetVirtualPath() const
	{
		return VirtualPath;
	}

	const UContentBrowserDataSource* GetDataSource() const 
	{
		return DataSource;
	}

private:
	/**
	 * Flags denoting the item type information for an item instance.
	 * @note This is always masked against EContentBrowserItemFlags::Type_MASK to remove any non-type information.
	 */
	EContentBrowserItemFlags ItemType = EContentBrowserItemFlags::None;

	/**
	 * The complete virtual path that uniquely identifies an item within its owner data source.
	 */
	FName VirtualPath;

	/** A pointer to the data source that manages the thing represented by this key */
	const UContentBrowserDataSource* DataSource = nullptr;
};

/**
 * Minimal representation of a FContentBrowserItemData instance that can be used as a map key.
 * @note This key doesn't consider the data source, so should only be used for maps within a given data source.
 *		 Use FContentBrowserItemKey for more general use where items may have come from different data sources.
 */
class FContentBrowserItemDataKey
{
public:
	/**
	 * Default constructor.
	 * Produces a key that is empty.
	 */
	FContentBrowserItemDataKey() = default;

	/**
	 * Construct this key from the given item.
	 */
	UE_API explicit FContentBrowserItemDataKey(const FContentBrowserItemData& InItemData);
	
	/**
	 * Construct this key from the given item type and virtual path.
	 */
	UE_API FContentBrowserItemDataKey(EContentBrowserItemFlags InItemType, FName InVirtualPath);

	/**
	 * Equality support.
	 */
	bool operator==(const FContentBrowserItemDataKey& InOther) const
	{
		return ItemType == InOther.ItemType
			&& VirtualPath.IsEqual(InOther.VirtualPath, ENameCase::CaseSensitive);
	}

	/**
	 * Inequality support.
	 */
	bool operator!=(const FContentBrowserItemDataKey& InOther) const
	{
		return !(*this == InOther);
	}

	/**
	 * Get the hash of the given instance.
	 */
	friend inline uint32 GetTypeHash(const FContentBrowserItemDataKey& InKey)
	{
		return HashCombine(GetTypeHash(InKey.ItemType), GetTypeHash(InKey.VirtualPath));
	}

protected:
	/**
	 * Flags denoting the item type information for an item instance.
	 * @note This is always masked against EContentBrowserItemFlags::Type_MASK to remove any non-type information.
	 */
	EContentBrowserItemFlags ItemType = EContentBrowserItemFlags::None;

	/**
	 * The complete virtual path that uniquely identifies an item within its owner data source.
	 */
	FName VirtualPath;
};

/**
 * Type describing an update to an item.
 */
class FContentBrowserItemDataUpdate
{
public:
	/**
	 * Create an item update for when a new item is added.
	 */
	static UE_API FContentBrowserItemDataUpdate MakeItemAddedUpdate(FContentBrowserItemData InItemData);

	/**
	 * Create an item update for when an existing item is updated.
	 */
	static UE_API FContentBrowserItemDataUpdate MakeItemModifiedUpdate(FContentBrowserItemData InItemData);

	/**
	 * Create an item update for when an existing item is moved (or renamed), including its previous virtual path.
	 */
	static UE_API FContentBrowserItemDataUpdate MakeItemMovedUpdate(FContentBrowserItemData InItemData, FName InPreviousVirtualPath);

	/**
	 * Create an item update for when an item is deleted.
	 */
	static UE_API FContentBrowserItemDataUpdate MakeItemRemovedUpdate(FContentBrowserItemData InItemData);

	/**
	 * Create an item update for when an item is unmounted.
	 */
	static UE_API FContentBrowserItemDataUpdate MakeItemUnmountedUpdate(FContentBrowserItemData InItemData);

	/**
	 * Get the value denoting the types of updates that can be emitted for an item.
	 */
	UE_API EContentBrowserItemUpdateType GetUpdateType() const;

	/**
	 * Get the item data for the update.
	 */
	UE_API const FContentBrowserItemData& GetItemData() const;

	/**
	 * Get the previous virtual path (UpdateType == Moved).
	 */
	UE_API FName GetPreviousVirtualPath() const;

private:
	/** Value denoting the types of updates that can be emitted for an item */
	EContentBrowserItemUpdateType UpdateType;

	/** Item data for the update */
	FContentBrowserItemData ItemData;

	/** Previous virtual path (UpdateType == Moved) */
	FName PreviousVirtualPath;
};

/**
 * The data sink interface that can be used to communicate with the Content Browser Data Subsystem.
 */
class IContentBrowserItemDataSink
{
public:
	/**
	 * Destructor.
	 */
	virtual ~IContentBrowserItemDataSink() = default;

	/**
	 * Queue an incremental item data update, for data sources that can provide delta-updates.
	 */
	virtual void QueueItemDataUpdate(FContentBrowserItemDataUpdate&& InUpdate) = 0;

	/**
	 * Notify a wholesale item data update, for data sources that can't provide delta-updates.
	 */
	 UE_DEPRECATED(5.5, "NotifyItemDataRefreshed is deprecated, for editor performance reasons no external systems should be able to request a full refresh.")
	virtual void NotifyItemDataRefreshed() = 0;

	/**
	 * Converts an internal path to a virtual path.
	 */
	virtual void ConvertInternalPathToVirtual(const FStringView InPath, FStringBuilderBase& OutPath) = 0;
};

#undef UE_API
