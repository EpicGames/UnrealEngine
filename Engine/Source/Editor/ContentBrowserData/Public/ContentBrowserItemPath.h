// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDataSubsystem.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Misc/AssertionMacros.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"

#include "ContentBrowserItemPath.generated.h"

#define UE_API CONTENTBROWSERDATA_API

class UObject;
struct FFrame;

/**
 * Hold multiple versions of a path as FNames
 * 
 * Path conversion each time Set is called
 */
USTRUCT(BlueprintType, meta=(HasNativeBreak="/Script/ContentBrowserData.ContentBrowserItemPathExtensions.BreakContentBrowserItemPath", HasNativeMake="/Script/ContentBrowserData.ContentBrowserItemPathExtensions.MakeContentBrowserItemPath"))
struct FContentBrowserItemPath
{
	GENERATED_BODY()

	UE_API FContentBrowserItemPath();

	FContentBrowserItemPath(const FName InVirtualPath, const FName InInternalPath)
		: VirtualPath(InVirtualPath)
		, InternalPath(InInternalPath)
	{
		check(!VirtualPath.IsNone());
		check(!InternalPath.IsNone());
	}

	UE_API FContentBrowserItemPath(const FStringView InPath, const EContentBrowserPathType InPathType);
	UE_API FContentBrowserItemPath(const TCHAR* InPath, const EContentBrowserPathType InPathType);
	UE_API FContentBrowserItemPath(const FName InPath, const EContentBrowserPathType InPathType);

	bool operator==(const FContentBrowserItemPath& Other) const 
	{ 
		return VirtualPath == Other.VirtualPath && InternalPath == Other.InternalPath;
	}

	/**
	 * Set the path being stored
	 */
	UE_API void SetPathFromString(const FStringView InPath, const EContentBrowserPathType InPathType);

	/**
	 * Set the path being stored
	 */
	UE_API void SetPathFromName(const FName InPath, const EContentBrowserPathType InPathType);

	/**
	 * Returns virtual path as FName (eg, "/All/Plugins/PluginA/MyFile").
	 */
	UE_API FName GetVirtualPathName() const;

	/**
	 * Returns internal path if there is one (eg,. "/PluginA/MyFile").
	 */
	UE_API FName GetInternalPathName() const;

	/**
	 * Returns virtual path as newly allocated FString (eg, "/All/Plugins/PluginA/MyFile").
	 */
	UE_API FString GetVirtualPathString() const;

	/**
	 * Returns internal path as newly allocated FString if there is one or empty FString (eg,. "/PluginA/MyFile").
	 */
	UE_API FString GetInternalPathString() const;

	/**
	 * Returns true if there is an internal path
	 */
	UE_API bool HasInternalPath() const;

private:
	/** Path as virtual (eg, "/All/Plugins/PluginA/MyFile") */
	FName VirtualPath;

	/** Path as internal (eg,. "/PluginA/MyFile") */
	FName InternalPath;
};

UCLASS()
class UContentBrowserItemPathExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Set the path being stored
	 */
	UFUNCTION(BlueprintCallable, Category = "Content Browser", meta = (ScriptMethod))
	static void SetPath(UPARAM(ref) FContentBrowserItemPath& ItemPath, const FName InPath, const EContentBrowserPathType InPathType)
	{
		ItemPath.SetPathFromName(InPath, InPathType);
	}

	/**
	 * Returns virtual path as FName (eg, "/All/Plugins/PluginA/MyFile").
	 */
	UFUNCTION(BlueprintCallable, Category = "Content Browser", meta = (ScriptMethod))
	static FName GetVirtualPath(const FContentBrowserItemPath& ItemPath)
	{
		return ItemPath.GetVirtualPathName();
	}

	/**
	 * Returns internal path if there is one (eg,. "/PluginA/MyFile").
	 */
	UFUNCTION(BlueprintCallable, Category = "Content Browser", meta = (ScriptMethod))
	static FName GetInternalPath(const FContentBrowserItemPath& ItemPath)
	{
		return ItemPath.HasInternalPath() ? ItemPath.GetInternalPathName() : NAME_None;
	}

	UFUNCTION(BlueprintPure, Category = "Content Browser", meta = (Keywords = "construct build", NativeMakeFunc))
	static FContentBrowserItemPath MakeContentBrowserItemPath(const FName InPath, const EContentBrowserPathType InPathType)
	{
		return FContentBrowserItemPath(InPath, InPathType);
	}

	UFUNCTION(BlueprintPure, Category = "Content Browser", meta = (NativeBreakFunc))
	static void BreakContentBrowserItemPath(const FContentBrowserItemPath& ItemPath, FName& VirtualPath, FName& InternalPath)
	{
		VirtualPath = ItemPath.GetVirtualPathName();
		InternalPath = ItemPath.HasInternalPath() ? ItemPath.GetInternalPathName() : NAME_None;
	}
 };

#undef UE_API
