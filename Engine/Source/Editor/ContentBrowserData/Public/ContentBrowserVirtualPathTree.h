// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StringFwd.h"
#include "ContentBrowserDataSubsystem.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/StringBuilder.h"
#include "UObject/NameTypes.h"

#define UE_API CONTENTBROWSERDATA_API

class FString;
template <typename FuncType> class TFunctionRef;

/**
 * Tree of virtual paths ending where internal paths start. Used for conversion of paths and during enumerate.
 */
class FContentBrowserVirtualPathTree
{
public:
	/** Adds the specified path to the tree, creating nodes as needed and calling OnPathAdded for any new paths added. Returns true if the specified path was actually added (as opposed to already existed) */
	UE_API bool CachePath(FName Path, FName InternalPath, TFunctionRef<void(FName)> OnPathAdded);

	/** Removes the specified path from the tree, calling OnPathRemoved for any existing paths removed. Returns true if the path was found and removed. */
	UE_API bool RemovePath(FName Path, TFunctionRef<void(FName)> OnPathRemoved);

	/** Checks whether the given path is one that we know about */
	UE_API bool PathExists(FName Path, bool& bIsFullyVirtual) const;

	/** Checks whether the given path is one that we know about */
	bool PathExists(FName Path) const
	{
		bool bHasChildren;
		return PathExists(Path, bHasChildren);
	}

	/** Get all of the paths we know about */
	UE_API bool GetAllPaths(TSet<FName>& OutPaths) const;

	/** Enumerate all of the paths we know about */
	UE_API void EnumerateAllPaths(TFunctionRef<bool(FName)> Callback) const;

	/** Recursively gathers all child paths from the specified base path relative to this node */
	UE_API bool GetSubPaths(FName BasePath, TSet<FName>& OutPaths, bool bRecurse = true) const;

	/** Recursively enumerates all child paths from the specified base path relative to this node */
	UE_API bool EnumerateSubPaths(FName BasePath, TFunctionRef<bool(FName, FName)> Callback, bool bRecurse = true) const;

	SIZE_T GetAllocatedSize(void) const
	{
		SIZE_T AllocatedSize = ParentPathToChildPaths.GetAllocatedSize() + ChildPathToParentPath.GetAllocatedSize() + VirtualToInternalMounts.GetAllocatedSize();

		for (const TPair<FName, TSet<FName>>& Pair : ParentPathToChildPaths)
		{
			AllocatedSize += Pair.Value.GetAllocatedSize();
		}

		return AllocatedSize;
	}

	/** Clears all paths */
	UE_API void Reset();

	/** Returns name of parent virtual path when given a virtual path */
	UE_API FName GetParentPath(FName Path) const;

	/** Returns number of paths in total */
	UE_API int32 NumPaths() const;

	/** Returns reference to map that allows looking up an internal path when given a virtual path */
	const TMap<FName, FName>& GetVirtualToInternalMounts() const
	{
		return VirtualToInternalMounts;
	}
	
	/**
	 * @note Use FPathViews::GetMountPointNameFromPath instead
	 * Returns name of the first folder in a path
	 * Removes starting forward slash and Classes_ prefix 
	 * Example: "/Classes_A/Textures" returns "A" and sets bOutHadClassesPrefix=true
	 */
	static UE_API FStringView GetMountPointFromPath(const FStringView InPath, bool& bOutHadClassesPrefix);

	/** Tries to converts virtual path into an internal path.
	 * Returns None if virtual prefix portion of the path does not exist in this path tree
	 * Returns Virtual if virtual prefix portion of the path exists in this path tree and there is no text past the virtual prefix portion
	 * Returns Internal if virtual prefix portion of the path exists in this path tree and there is text afterwards
	 * @Note: When Internal is the return value it does not check to see if the internal path actually exists or not
	 */
	UE_API EContentBrowserPathType TryConvertVirtualPathToInternal(FStringView InPath, FStringBuilderBase& OutPath) const;
	UE_API EContentBrowserPathType TryConvertVirtualPathToInternal(FStringView InPath, FString& OutPath) const;
	UE_API EContentBrowserPathType TryConvertVirtualPathToInternal(FName InPath, FName& OutPath) const;

private:

	/** Is the user option to show the all folder enabled */
	UE_API bool IsShowAllFolderEnabled() const;

	/** Get the string to use when show all folder is enabled */
	UE_API const FString& GetAllFolderPrefix() const;

	/** A one-to-one mapping between a virtual path and internal path */
	TMap<FName, FName> VirtualToInternalMounts;

	/** A one-to-many mapping between a parent path and its child paths. */
	TMap<FName, TSet<FName>> ParentPathToChildPaths;

	/** A one-to-one mapping between a child path and its parent path. Paths without a parent (root paths) will not appear in this map. */
	TMap<FName, FName> ChildPathToParentPath;
};

#undef UE_API
