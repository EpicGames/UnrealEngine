// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/Function.h"
#include "Templates/Invoke.h"

/**
 * Utility class to simplify implementation of data sources that can be called as 'get'
 * (fill an array) or 'enumerate' (call a callback).
 * 
 * Usage: Functions should take TGetOrEnumerateSink by value or const ref to allow in-place construction.
 * 
 * @tparam ItemType The type of item produced. The sink can be constructed with TArray<ItemType> or TFunctionRef<ItemType&&>.
 */
template<typename ItemType>
class TGetOrEnumerateSink
{
public:
    explicit TGetOrEnumerateSink(TFunctionRef<bool(ItemType&&)> InCallback)
        : Callback(MoveTemp(InCallback))
    {
    }
    explicit TGetOrEnumerateSink(TArray<ItemType>& InOutput)
        : Array(&InOutput)
    {
    }

    // Allow copying for delegation to other methods if desired. Just copies pointers into sink. No state needs to be maintained between copies. 
    TGetOrEnumerateSink(const TGetOrEnumerateSink&) = default;
    TGetOrEnumerateSink& operator=(const TGetOrEnumerateSink&) = default;
    
    // Allow moving to delegate the sink into other calls 
    TGetOrEnumerateSink(TGetOrEnumerateSink&&) = default;
    TGetOrEnumerateSink& operator=(TGetOrEnumerateSink&&) = default;

    void ReserveMore(int32 NumAdditionalItems) const
    {
        if (Array)
        {
            Array->Reserve(Array->Num() + NumAdditionalItems);
        }
    }

    bool ProduceItem(ItemType&& Item) const
    {
        if (Array)
        {
            Array->Add(MoveTemp(Item));
            return true;
        }
        else
        {
            return ::Invoke(Callback.GetValue(), MoveTemp(Item));
        }
    }

private:
    TArray<ItemType>* Array = nullptr;
    TOptional<TFunctionRef<bool(ItemType&&)>> Callback;
};