// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"

class UContentBrowserDataSubsystem;

class IContentBrowserDataModule : public IModuleInterface
{
public:
	virtual ~IContentBrowserDataModule() = default;

	/**
	 * Singleton-like access to this module's interface.
	 * @note Beware of calling this during the shutdown phase. Your module might have been unloaded already (@see GetPtr).
	 *
	 * @return Returns the singleton instance, loading the module on demand if needed.
	 */
	static IContentBrowserDataModule& Get()
	{
		static const FName ModuleName = "ContentBrowserData";
		return FModuleManager::LoadModuleChecked<IContentBrowserDataModule>(ModuleName);
	}

	/**
	 * Singleton-like access to this module's interface.
	 *
	 * @return Returns singleton instance, or null if the module isn't loaded or has been unloaded.
	 */
	static IContentBrowserDataModule* GetPtr()
	{
		static const FName ModuleName = "ContentBrowserData";
		return FModuleManager::GetModulePtr<IContentBrowserDataModule>(ModuleName);
	}

	virtual UContentBrowserDataSubsystem* GetSubsystem() const = 0;
};
